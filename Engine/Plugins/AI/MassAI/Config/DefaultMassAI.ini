[CoreRedirects]
+FunctionRedirects=(OldName="/Script/MassAIBehavior.MassLookAtSubsystem.CreateRequest",NewName="/Script/MassAIBehavior.MassLookAtSubsystem.CreateLookAtPositionRequest")
