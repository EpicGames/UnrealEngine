{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.4",
	"FriendlyName": "MassAI",
	"Description": "AI-specific functionality extending MassGameplay",
	"Category": "AI",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "MassNavigation",
			"Type": "Runtime"
		},
		{
			"Name": "MassNavMeshNavigation",
			"Type": "Runtime"
		},
		{
			"Name": "MassZoneGraphNavigation",
			"Type": "Runtime"
		},
		{
			"Name": "MassNavigationEditor",
			"Type": "Editor",
			"LoadingPhase": "PostEngineInit"
		},
		{
			"Name": "MassAIBehavior",
			"Type": "Runtime"
		},
		{
			"Name": "MassAIBehaviorEditor",
			"Type": "Editor"
		},
		{
			"Name": "MassAIReplication",
			"Type": "Runtime"
		},
		{
			"Name": "MassAIDebug",
			"Type": "Runtime"
		},
		{
			"Name": "MassAITestSuite",
			"Type": "UncookedOnly"
		}
	],
	"Plugins": [
		{
			"Name": "MassGameplay",
			"Enabled": true
		},
		{
			"Name": "ZoneGraph",
			"Enabled": true
		},
		{
			"Name": "ZoneGraphAnnotations",
			"Enabled": true
		},
		{
			"Name": "SmartObjects",
			"Enabled": true
		},
		{
			"Name": "StateTree",
			"Enabled": true
		},
		{
			"Name": "NavCorridor",
			"Enabled": true
		}
	]
}