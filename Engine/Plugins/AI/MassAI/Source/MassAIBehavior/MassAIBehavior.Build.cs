// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassAIBehavior : ModuleRules
	{
		public MassAIBehavior(ReadOnlyTargetRules Target) : base(Target)
		{
			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PrivateIncludePathModuleNames.AddRange(
				new string[] {
					"AIModule"
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"MassCore",
					"MassEntity",
					"Core",
					"CoreUObject",
					"Engine",
					"DeveloperSettings",
					"GameplayTags",
					"MassActors",
					"MassCommon",
					"MassLOD",
					"MassMovement",
					"MassNavigation",
					"MassNavMeshNavigation",
					"MassZoneGraphNavigation",
					"MassRepresentation",
					"MassSignals",
					"MassSmartObjects",
					"MassSpawner",
					"MassSimulation",
					"NavigationSystem",
					"SmartObjectsModule",
					"StateTreeModule",
					"ZoneGraph",
					"ZoneGraphAnnotations",
					"MassGameplayExternalTraits",
					"NavCorridor"
				}
			);

			PrivateDependencyModuleNames.AddRange(
				new string[] {
					"PropertyBindingUtils"
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("EditorFramework");
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}
