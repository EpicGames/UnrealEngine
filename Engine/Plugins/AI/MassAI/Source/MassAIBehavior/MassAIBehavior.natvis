<?xml version="1.0" encoding="utf-8"?>
<AutoVisualizer xmlns="http://schemas.microsoft.com/vstudio/debugger/natvis/2010">

  <!--
  *
  * MassAIBehavior Visualizers
  *
  -->

  <Type Name="UMassLookAtSubsystem::FRequest">
    <DisplayString>Params={{ {Parameters} }}, Request={RequestHandle}</DisplayString>
  </Type>

  <Type Name="FMassLookAtRequestFragment">
    <DisplayString Condition="LookAtMode == EMassLookAtMode::LookAtEntity">Priority={Priority}, Mode={LookAtMode}, Viewer={ViewerEntity}, Target={TargetEntity}</DisplayString>
    <DisplayString>Priority={Priority}, Mode={LookAtMode}, Viewer={ViewerEntity}</DisplayString>
  </Type>

</AutoVisualizer>
