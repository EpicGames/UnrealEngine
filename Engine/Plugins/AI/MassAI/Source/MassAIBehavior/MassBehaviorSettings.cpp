// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassBehaviorSettings.h"
#include "MassStateTreeProcessors.h"

//----------------------------------------------------------------------//
// UMassBehaviorSettings
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassBehaviorSettings)

UMassBehaviorSettings::UMassBehaviorSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Default values.
	MaxActivationsPerLOD[EMassLOD::High] = 100;
	MaxActivationsPerLOD[EMassLOD::Medium] = 100;
	MaxActivationsPerLOD[EMassLOD::Low] = 100;
	MaxActivationsPerLOD[EMassLOD::Off] = 100;

	DynamicStateTreeProcessorClass = UMassStateTreeProcessor::StaticClass();
}
