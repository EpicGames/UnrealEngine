// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/ZoneGraphTagConditions.h"
#include "StateTreeExecutionContext.h"

#if WITH_EDITOR
#include "ZoneGraphSettings.h"
#endif// WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(ZoneGraphTagConditions)

#if WITH_EDITOR
#define LOCTEXT_NAMESPACE "ZoneGraphTagConditions"

namespace UE::MassBehavior::ZoneGraph
{
	FText GetTagName(const FZoneGraphTag Tag)
	{
		const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
		check(ZoneGraphSettings);
		TConstArrayView<FZoneGraphTagInfo> TagInfos = ZoneGraphSettings->GetTagInfos();
		
		for (const FZoneGraphTagInfo& TagInfo : TagInfos)
		{
			if (TagInfo.Tag == Tag)
			{
				return FText::FromName(TagInfo.Name);
			}
		}
		return FText::GetEmpty();
	}

	FText GetTagMaskName(const FZoneGraphTagMask TagMask)
	{
		const UZoneGraphSettings* ZoneGraphSettings = GetDefault<UZoneGraphSettings>();
		check(ZoneGraphSettings);
		TConstArrayView<FZoneGraphTagInfo> TagInfos = ZoneGraphSettings->GetTagInfos();

		TArray<FText> Names;
		for (const FZoneGraphTagInfo& Info : TagInfos)
		{
			if (TagMask.Contains(Info.Tag))
			{
				if (Info.IsValid())
				{
					Names.Add(FText::FromName(Info.Name));
				}
			}
		}
		if (Names.Num() == 0)
		{
			return LOCTEXT("EmptyMask", "(Empty)");
		}
		
		if (Names.Num() > 2)
		{
			Names.SetNum(2);
			Names.Add(FText::FromString(TEXT("...")));
		}
		
		return FText::Join(FText::FromString(TEXT(", ")), Names);
	}

	FText GetMaskOperatorText(const EZoneLaneTagMaskComparison Operator)
	{
		switch (Operator)
		{
		case EZoneLaneTagMaskComparison::Any:
			return LOCTEXT("ContainsAny", "Any");
		case EZoneLaneTagMaskComparison::All:
			return LOCTEXT("ContainsAll", "All");
		case EZoneLaneTagMaskComparison::Not:
			return LOCTEXT("ContainsNot", "Not");
		default:
			break;
		}
		return FText::FromString(TEXT("??"));
	}

}

#endif// WITH_EDITOR


//----------------------------------------------------------------------//
//  FZoneGraphTagFilterCondition
//----------------------------------------------------------------------//

bool FZoneGraphTagFilterCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	return Filter.Pass(InstanceData.Tags) ^ bInvert;
}


//----------------------------------------------------------------------//
//  FZoneGraphTagMaskCondition
//----------------------------------------------------------------------//

bool FZoneGraphTagMaskCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	return InstanceData.Left.CompareMasks(InstanceData.Right, Operator) ^ bInvert;
}


//----------------------------------------------------------------------//
//  FZoneGraphTagCondition
//----------------------------------------------------------------------//

bool FZoneGraphTagCondition::TestCondition(FStateTreeExecutionContext& Context) const
{
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	return (InstanceData.Left == InstanceData.Right) ^ bInvert;
}


#if WITH_EDITOR
#undef LOCTEXT_NAMESPACE
#endif // WITH_EDITOR
