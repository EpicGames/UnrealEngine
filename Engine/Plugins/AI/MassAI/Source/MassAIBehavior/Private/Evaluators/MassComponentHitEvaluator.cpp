// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluators/MassComponentHitEvaluator.h"
#include "MassAIBehaviorTypes.h"
#include "MassComponentHitSubsystem.h"
#include "MassStateTreeDependency.h"
#include "MassStateTreeExecutionContext.h"
#include "Engine/World.h"
#include "StateTreeLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassComponentHitEvaluator)

bool FMassComponentHitEvaluator::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(ComponentHitSubsystemHandle);

	return true;
}

void FMassComponentHitEvaluator::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly(ComponentHitSubsystemHandle);
}

void FMassComponentHitEvaluator::Tick(FStateTreeExecutionContext &Context, const float DeltaTime) const
{
	// Look for recent hits
	const UMassComponentHitSubsystem& HitSubsystem = Context.GetExternalData(ComponentHitSubsystemHandle);
	const FMassHitResult* HitResult = HitSubsystem.GetLastHit(static_cast<FMassStateTreeExecutionContext&>(Context).GetEntity());

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	
	// LastHitEntity is not reset intentionally, so that it's available the duration of the behavior reacting to it.
	InstanceData.bGotHit = false;

	// If the hit is very recent, set the got hit, and update last hit entity.
	if (HitResult != nullptr)
	{
		// @todo: This is a bit of a kludge to expose an event to StateTree.
		const UWorld* World = Context.GetWorld();
		check(World);
		const double CurrentTime = World->GetTimeSeconds();
		const double TimeSinceHit = CurrentTime - HitResult->HitTime;
		constexpr double HitEventDuration = 0.1;
		if (TimeSinceHit < HitEventDuration)
		{
			MASSBEHAVIOR_LOG(VeryVerbose, TEXT("Got hit"));
			InstanceData.bGotHit = true;
			InstanceData.LastHitEntity = HitResult->OtherEntity;
		}
	}
}
