// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluators/MassZoneGraphAnnotationEvaluator.h"
#include "MassStateTreeExecutionContext.h"
#include "MassZoneGraphAnnotationFragments.h"
#include "StateTreeLinker.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphAnnotationEvaluator)


FMassZoneGraphAnnotationEvaluator::FMassZoneGraphAnnotationEvaluator()
{
}

bool FMassZoneGraphAnnotationEvaluator::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(AnnotationTagsFragmentHandle);

	return true;
}

void FMassZoneGraphAnnotationEvaluator::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly<FMassZoneGraphAnnotationFragment>();
}

void FMassZoneGraphAnnotationEvaluator::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	const FMassZoneGraphAnnotationFragment& AnnotationTagsFragment = Context.GetExternalData(AnnotationTagsFragmentHandle);
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	InstanceData.AnnotationTags = AnnotationTagsFragment.Tags;
}
