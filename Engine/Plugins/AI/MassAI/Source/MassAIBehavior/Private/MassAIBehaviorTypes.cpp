// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassAIBehaviorTypes.h"

DEFINE_LOG_CATEGORY(LogMassBehavior);

