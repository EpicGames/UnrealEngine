// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassComponentHitSubsystem.h"

#include "GameFramework/Pawn.h"
#include "MassAgentComponent.h"
#include "MassAgentSubsystem.h"
#include "MassSignalSubsystem.h"
#include "MassSimulationSubsystem.h"
#include "Components/CapsuleComponent.h"
#include "Subsystems/SubsystemCollection.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassComponentHitSubsystem)

namespace UE::MassComponentHit
{

bool bOnlyProcessHitsFromPlayers = true;

FAutoConsoleVariableRef ConsoleVariables[] =
{
	FAutoConsoleVariableRef(
		TEXT("ai.mass.OnlyProcessHitsFromPlayers"),
		bOnlyProcessHitsFromPlayers,
		TEXT("Activates extra filtering to ignore hits from actors that are not controlled by the player."),
		ECVF_Cheat)
};

} // UE::MassComponentHit

void UMassComponentHitSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	Collection.InitializeDependency<UMassSimulationSubsystem>();

	SignalSubsystem = Collection.InitializeDependency<UMassSignalSubsystem>();
	checkfSlow(SignalSubsystem != nullptr, TEXT("MassSignalSubsystem is required"));

	AgentSubsystem = Collection.InitializeDependency<UMassAgentSubsystem>();
	checkfSlow(AgentSubsystem != nullptr, TEXT("MassAgentSubsystem is required"));

	AgentSubsystem->GetOnMassAgentComponentEntityAssociated().AddLambda([this](const UMassAgentComponent& AgentComponent)
	{
		if (UCapsuleComponent* CapsuleComponent = AgentComponent.GetOwner()->FindComponentByClass<UCapsuleComponent>())
		{
			RegisterForComponentHit(AgentComponent.GetEntityHandle(), *CapsuleComponent);
		}
	});

	AgentSubsystem->GetOnMassAgentComponentEntityDetaching().AddLambda([this](const UMassAgentComponent& AgentComponent)
	{
		if (UCapsuleComponent* CapsuleComponent = AgentComponent.GetOwner()->FindComponentByClass<UCapsuleComponent>())
		{
			UnregisterForComponentHit(AgentComponent.GetEntityHandle(), *CapsuleComponent);
		}
	});

	OverrideSubsystemTraits<UMassComponentHitSubsystem>(Collection);
}

void UMassComponentHitSubsystem::Deinitialize()
{
	checkfSlow(AgentSubsystem != nullptr, TEXT("MassAgentSubsystem must have be set during initialization"));
	AgentSubsystem->GetOnMassAgentComponentEntityAssociated().RemoveAll(this);
	AgentSubsystem->GetOnMassAgentComponentEntityDetaching().RemoveAll(this);

	Super::Deinitialize();
}

void UMassComponentHitSubsystem::RegisterForComponentHit(const FMassEntityHandle Entity, UCapsuleComponent& CapsuleComponent)
{
	EntityToComponentMap.Add(Entity, &CapsuleComponent);
	ComponentToEntityMap.Add(&CapsuleComponent, Entity);
	CapsuleComponent.OnComponentHit.AddDynamic(this, &UMassComponentHitSubsystem::OnHitCallback);
}

void UMassComponentHitSubsystem::UnregisterForComponentHit(const FMassEntityHandle Entity, UCapsuleComponent& CapsuleComponent)
{
	EntityToComponentMap.Remove(Entity);
	ComponentToEntityMap.Remove(&CapsuleComponent);
	CapsuleComponent.OnComponentHit.RemoveAll(this);
}

void UMassComponentHitSubsystem::OnHitCallback(UPrimitiveComponent* HitComp, AActor* OtherActor, UPrimitiveComponent* OtherComp, FVector NormalImpulse, const FHitResult& Hit)
{
	const UWorld* World = GetWorld();
	check(World);
	const FMassEntityHandle Entity = ComponentToEntityMap.FindChecked(HitComp);
	FMassEntityHandle* OtherEntity = ComponentToEntityMap.Find(OtherComp);

	bool bProcessHit = (OtherEntity != nullptr && OtherEntity->IsSet());
	if (bProcessHit && UE::MassComponentHit::bOnlyProcessHitsFromPlayers)
	{
		const APawn* HitActorAsPawn = (HitComp != nullptr) ? Cast<APawn>(HitComp->GetOwner()) : nullptr;
		const APawn* OtherAsPawn = Cast<APawn>(OtherActor);
		bProcessHit = (HitActorAsPawn != nullptr && HitActorAsPawn->IsPlayerControlled()) || (OtherAsPawn != nullptr && OtherAsPawn->IsPlayerControlled());
	}

	const double CurrentTime = World->GetTimeSeconds();

	// If new hit result comes during this duration, it will be merged to existing one.
	constexpr double HitResultMergeDuration = 1.;
	if (bProcessHit)
	{
		FMassHitResult* ExistingHitResult = HitResults.Find(Entity);
		if (ExistingHitResult)
		{
			const double TimeSinceLastHit = CurrentTime - ExistingHitResult->LastFilteredHitTime;
			if (TimeSinceLastHit < HitResultMergeDuration)
			{
				ExistingHitResult->LastFilteredHitTime = CurrentTime;
				bProcessHit = false;
			}
		}
	}

	if (bProcessHit)
	{
		HitResults.Add(Entity, {*OtherEntity, CurrentTime});

		checkfSlow(SignalSubsystem != nullptr, TEXT("MassSignalSubsystem must have be set during initialization"));
		SignalSubsystem->SignalEntity(UE::Mass::Signals::HitReceived, Entity);
	}
}

const FMassHitResult* UMassComponentHitSubsystem::GetLastHit(const FMassEntityHandle Entity) const
{
	return HitResults.Find(Entity);
}

void UMassComponentHitSubsystem::Tick(float DeltaTime)
{
	const UWorld* World = GetWorld();
	check(World);

	const double CurrentTime = World->GetTimeSeconds();
	constexpr double HitResultDecayDuration = 1.;
	
	for (auto Iter = HitResults.CreateIterator(); Iter; ++Iter)
	{
		const FMassHitResult& HitResult = Iter.Value();
		const double ElapsedTime = CurrentTime - HitResult.LastFilteredHitTime;
		if (ElapsedTime > HitResultDecayDuration)
		{
			Iter.RemoveCurrent();
		}
	}
}

TStatId UMassComponentHitSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMassComponentHitSubsystem, STATGROUP_Tickables);
}
