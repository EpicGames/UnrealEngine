// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtFragments.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphTypes.h"
#include "Curves/BezierUtilities.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLookAtFragments)

FVector FMassLookAtTrajectoryFragment::GetPointAtDistanceExtrapolated(const float DistanceAlongPath) const
{
	if (NumPoints == 0)
	{
		return FVector::ZeroVector;
	}

	if (NumPoints == 1)
	{
		const float ExtrapolatedDistance = DistanceAlongPath - Points[0].DistanceAlongLane.Get();
		const FMassLookAtTrajectoryPoint& Point = Points[0];
		return Point.Position + FVector(Point.Tangent.Get() * ExtrapolatedDistance, 0.0f);
	}

	// Extrapolate along tangents if out of bounds.
	const float StartDistance = Points[0].DistanceAlongLane.Get();
	if (DistanceAlongPath < StartDistance)
	{
		const float ExtrapolatedDistance = DistanceAlongPath - StartDistance;
		const FMassLookAtTrajectoryPoint& Point = Points[0];
		return Point.Position + FVector(Point.Tangent.Get() * ExtrapolatedDistance, 0.0f);
	}

	const int32 LastPointIndex = NumPoints - 1;
	const float EndDistance = Points[LastPointIndex].DistanceAlongLane.Get();
	if (DistanceAlongPath > EndDistance)
	{
		const float ExtrapolatedDistance = DistanceAlongPath - EndDistance;
		const FMassLookAtTrajectoryPoint& Point = Points[LastPointIndex];
		return Point.Position + FVector(Point.Tangent.Get() * ExtrapolatedDistance, 0.0f);
	}

	check(NumPoints >= 2);

	// Find segment 
	int32 SegmentIndex = 0;
	while (SegmentIndex < ((int32)NumPoints - 2))
	{
		const float SegmentEndDistance = Points[SegmentIndex + 1].DistanceAlongLane.Get();
		if (DistanceAlongPath < SegmentEndDistance)
		{
			break;
		}
		SegmentIndex++;
	}

	check(SegmentIndex >= 0 && SegmentIndex <= (int32)NumPoints - 2);

	// Interpolate
	const FMassLookAtTrajectoryPoint& StartPoint = Points[SegmentIndex];
	const FMassLookAtTrajectoryPoint& EndPoint = Points[SegmentIndex + 1];

	const float SegStartDistance = StartPoint.DistanceAlongLane.Get();
	const float SegEndDistance = EndPoint.DistanceAlongLane.Get();
	const float SegLength = SegEndDistance - SegStartDistance;
	const float InvSegLength = SegLength > KINDA_SMALL_NUMBER ? 1.0f / SegLength : 0.0f;
	const float T = FMath::Clamp((DistanceAlongPath - SegStartDistance) * InvSegLength, 0.0f, 1.0f);

	// 1/3 third is used to create smooth bezier curve. On linear segments 1/3 will result linear interpolation.
	const FVector::FReal TangentDistance = FVector::Dist(StartPoint.Position, EndPoint.Position) / 3.;
	const FVector StartControlPoint = StartPoint.Position + StartPoint.Tangent.GetVector() * TangentDistance;
	const FVector EndControlPoint = EndPoint.Position - EndPoint.Tangent.GetVector() * TangentDistance;

	return UE::CubicBezier::Eval(StartPoint.Position, StartControlPoint, EndControlPoint, EndPoint.Position, T);
}
