// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtProcessors.h"
#include "Algo/RandomShuffle.h"
#include "Curves/BezierUtilities.h"
#include "MassAIBehaviorTypes.h"
#include "MassCommonFragments.h"
#include "MassDebugger.h"
#include "MassLODFragments.h"
#include "MassLookAtFragments.h"
#include "MassLookAtSubsystem.h"
#include "MassMovementFragments.h"
#include "MassNavigationFragments.h"
#include "MassRepresentationTypes.h"
#include "MassZoneGraphNavigationFragments.h"
#include "VisualLogger/VisualLogger.h"
#include "ZoneGraphQuery.h"
#include "ZoneGraphSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLookAtProcessors)

namespace UE::MassBehavior
{
	namespace Tweakables
	{
		float TrajectoryLookAhead = 600.f;
	}

	FAutoConsoleVariableRef CVars[] =
	{
		FAutoConsoleVariableRef(TEXT("ai.mass.LookAt.TrajectoryLookAhead"), Tweakables::TrajectoryLookAhead,
								TEXT("Distance (in cm) further along the look at trajectory (based on current path) to look at while moving."), ECVF_Cheat),
	};

	// Clamps direction vector to a cone specified by the cone angle along X-axis
	FVector ClampDirectionToXAxisCone(const FVector Direction, const float ConeAngle)
	{
		FVector::FReal ConeSin = 0., ConeCos = 0.;
		FMath::SinCos(&ConeSin, &ConeCos, ConeAngle);
		
		const FVector::FReal AngleCos = Direction.X; // Same as FVector::DotProduct(FVector::ForwardVector, Direction);
		if (AngleCos < ConeCos)
		{
			const FVector::FReal DistToRimSq = FMath::Square(Direction.Y) + FMath::Square(Direction.Z);
			const FVector::FReal InvDistToRim = DistToRimSq > KINDA_SMALL_NUMBER ? (1. / FMath::Sqrt(DistToRimSq)) : 0.;
			return FVector(ConeCos, Direction.Y * InvDistToRim * ConeSin, Direction.Z * InvDistToRim * ConeSin);
		}
		
		return Direction;
	}

	float GazeEnvelope(const float GazeTime, const float GazeDuration, const EMassLookAtGazeMode Mode)
	{
		if (GazeDuration < KINDA_SMALL_NUMBER || Mode == EMassLookAtGazeMode::None)
		{
			return 0.0f;
		}

		if (Mode == EMassLookAtGazeMode::Constant)
		{
			return 1.0;
		}

		// @todo LookAt: make configurable
		const float SustainTime = GazeDuration * 0.25f;
		const float DecayTime = GazeDuration * 0.45f;
		
		if (GazeTime < SustainTime)
		{
			return 1.0f;
		}
		if (GazeTime > DecayTime)
		{
			return 0.0f;
		}
		
		const float Duration = FMath::Max(KINDA_SMALL_NUMBER, DecayTime - SustainTime);
		const float NormTime = FMath::Clamp((GazeTime - SustainTime) / Duration, 0.0f, 1.0f);
		return 1.0f - NormTime;
	}

}// namespace UE::MassBehavior


//----------------------------------------------------------------------//
// UMassLookAtProcessor
//----------------------------------------------------------------------//
UMassLookAtProcessor::UMassLookAtProcessor()
	: EntityQuery_Conditional(*this)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Tasks;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Representation);
}

void UMassLookAtProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery_Conditional.AddRequirement<FMassLookAtFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery_Conditional.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddRequirement<FMassLookAtTrajectoryFragment>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddRequirement<FMassZoneGraphShortPathFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddTagRequirement<FMassMediumLODTag>(EMassFragmentPresence::None);
	EntityQuery_Conditional.AddTagRequirement<FMassLowLODTag>(EMassFragmentPresence::None);
	EntityQuery_Conditional.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery_Conditional.AddChunkRequirement<FMassVisualizationChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.SetChunkFilter(&FMassVisualizationChunkFragment::AreAnyEntitiesVisibleInChunk);
	EntityQuery_Conditional.AddSubsystemRequirement<UMassNavigationSubsystem>(EMassFragmentAccess::ReadOnly);
	EntityQuery_Conditional.AddSubsystemRequirement<UMassLookAtSubsystem>(EMassFragmentAccess::ReadOnly);
	EntityQuery_Conditional.AddSubsystemRequirement<UZoneGraphSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UMassLookAtProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(LookAtProcessor_Run);

	const double CurrentTime = GetWorld()->GetTimeSeconds();

	EntityQuery_Conditional.ForEachEntityChunk(Context, [this, &EntityManager, CurrentTime](FMassExecutionContext& Context)
		{
			const UMassNavigationSubsystem& MassNavSystem = Context.GetSubsystemChecked<UMassNavigationSubsystem>();
			const UMassLookAtSubsystem& LookAtTargetSystem = Context.GetSubsystemChecked<UMassLookAtSubsystem>();
			const UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetSubsystemChecked<UZoneGraphSubsystem>();

			const TArrayView<FMassLookAtFragment> LookAtList = Context.GetMutableFragmentView<FMassLookAtFragment>();
			const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
			const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();
			const TConstArrayView<FMassZoneGraphLaneLocationFragment> ZoneGraphLocationList = Context.GetFragmentView<FMassZoneGraphLaneLocationFragment>();
			const TConstArrayView<FMassZoneGraphShortPathFragment> ShortPathList = Context.GetFragmentView<FMassZoneGraphShortPathFragment>();
			const TArrayView<FMassLookAtTrajectoryFragment> LookAtTrajectoryList = Context.GetMutableFragmentView<FMassLookAtTrajectoryFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassLookAtFragment& LookAt = LookAtList[EntityIt];
				const FTransformFragment& TransformFragment = TransformList[EntityIt];

				const bool bHasLookAtTrajectory =
					MoveTargetList.Num() > 0
					&& ZoneGraphLocationList.Num() > 0
					&& LookAtTrajectoryList.Num() > 0
					&& ShortPathList.Num() > 0;

				bool bDisplayDebug = false;
				const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
	#if WITH_MASSGAMEPLAY_DEBUG
				FColor EntityColor = FColor::White;
				bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Entity, &EntityColor);
	#endif // WITH_MASSGAMEPLAY_DEBUG

				// Update gaze target when current cycle is finished.
				if (LookAt.RandomGazeMode != EMassLookAtGazeMode::None)
				{
					const double TimeSinceUpdate = CurrentTime - LookAt.GazeStartTime;
					if (TimeSinceUpdate >= LookAt.GazeDuration)
					{
						FindNewGazeTarget(MassNavSystem, LookAtTargetSystem, EntityManager, CurrentTime, TransformFragment.GetTransform(), Entity, LookAt);
					}
				}

				// Update specific look at mode.
				LookAt.Direction = FVector::ForwardVector;
				LookAt.MainTargetLocation = FVector::ZeroVector;

				switch (LookAt.LookAtMode)
				{
				case EMassLookAtMode::LookForward:
					// Empty, forward set already above.
					break;
					
				case EMassLookAtMode::LookAlongPath:
					if (bHasLookAtTrajectory)
					{
						const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
						const FMassZoneGraphLaneLocationFragment& ZoneGraphLocation = ZoneGraphLocationList[EntityIt];
						FMassLookAtTrajectoryFragment& LookAtTrajectory = LookAtTrajectoryList[EntityIt];

						if (MoveTarget.GetCurrentActionID() != LookAt.LastSeenActionID)
						{
							const FMassZoneGraphLaneLocationFragment& LaneLocation = ZoneGraphLocationList[EntityIt];
							const FMassZoneGraphShortPathFragment& ShortPath = ShortPathList[EntityIt];
							
							BuildTrajectory(ZoneGraphSubsystem, LaneLocation, ShortPath, Entity, bDisplayDebug, LookAtTrajectory);
							LookAt.LastSeenActionID = MoveTarget.GetCurrentActionID();
						}
						
						UpdateLookAtTrajectory(TransformFragment.GetTransform(), ZoneGraphLocation, LookAtTrajectory, bDisplayDebug, LookAt);
					}
					break;

				case EMassLookAtMode::LookAtEntity:
					UpdateLookAtTrackedEntity(EntityManager, TransformFragment.GetTransform(), bDisplayDebug, LookAt);
					break;
					
				default:
					break;
				}

				// Apply gaze
				if (LookAt.RandomGazeMode != EMassLookAtGazeMode::None)
				{
					const float TimeSinceUpdate = FloatCastChecked<float>(CurrentTime - LookAt.GazeStartTime, /* Precision */ 1./64.);
					const float GazeStrength = UE::MassBehavior::GazeEnvelope(TimeSinceUpdate, LookAt.GazeDuration, LookAt.RandomGazeMode);

					if (GazeStrength > KINDA_SMALL_NUMBER)
					{
						const bool bHasTarget = UpdateGazeTrackedEntity(EntityManager, TransformFragment.GetTransform(), bDisplayDebug, LookAt);

						if (bHasTarget)
						{
							// Treat target gaze as absolute direction.
							LookAt.Direction = FMath::Lerp(LookAt.Direction, LookAt.GazeDirection, GazeStrength).GetSafeNormal();
						}
						else
						{
							// Treat random offset as relative direction.
							const FQuat GazeRotation = FQuat::FindBetweenNormals(FVector::ForwardVector, FMath::Lerp(FVector::ForwardVector, LookAt.GazeDirection, GazeStrength).GetSafeNormal());
							LookAt.Direction = GazeRotation.RotateVector(LookAt.Direction);
						}
					}
				}

				// Clamp
				LookAt.Direction = UE::MassBehavior::ClampDirectionToXAxisCone(LookAt.Direction, FMath::DegreesToRadians(AngleThresholdInDegrees));

	#if WITH_MASSGAMEPLAY_DEBUG
				if (bDisplayDebug)
				{
					const FVector Origin = TransformFragment.GetTransform().GetLocation() + FVector(0.f,0.f,DebugZOffset);
					const FVector Dest = Origin + 100.f*TransformFragment.GetTransform().TransformVector(LookAt.Direction);
					UE_VLOG_ARROW(this, LogMassBehavior, Display, Origin, Dest, EntityColor, TEXT(""));
				}
	#endif // WITH_MASSGAMEPLAY_DEBUG
			}
		});
}

void UMassLookAtProcessor::FindNewGazeTarget(const UMassNavigationSubsystem& MassNavSystem
	, const UMassLookAtSubsystem& LookAtSystem
	, const FMassEntityManager& EntityManager
	, const double CurrentTime
	, const FTransform& Transform
	, const FMassEntityHandle Entity
	, FMassLookAtFragment& LookAt) const
{
	const FMassEntityHandle LastTrackedEntity = LookAt.GazeTrackedEntity;
	
	LookAt.GazeTrackedEntity.Reset();
	LookAt.GazeDirection = FVector::ForwardVector;
	LookAt.GazeTargetLocation = FVector::ZeroVector;

	// Search for potential targets in front
	bool bTargetFound = false;
	if (LookAt.bRandomGazeEntities)
	{
		const float CosAngleThreshold = FMath::Cos(FMath::DegreesToRadians(AngleThresholdInDegrees));
		const FVector Extent(QueryExtent, QueryExtent, QueryExtent);
		const FVector QueryOrigin = Transform.TransformPosition(FVector(0.5f*QueryExtent, 0.f, 0.f));
		const FBox QueryBox = FBox(QueryOrigin - 0.5f*Extent, QueryOrigin + 0.5f*Extent);

		// Process from LookAt targets grid
		TArray<UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType, TInlineAllocator<UE::Mass::LookAt::HashGridResultsSoftLimit>> NearbyEntities;
		if (LookAtSystem.Query(QueryBox, NearbyEntities))
		{
			NearbyEntities.Sort([]
			(const UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType& A, const UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType& B)
			{
				return A.Priority < B.Priority;
			});

			TOptional<uint8> LastPriority;
			TArray<int32, TInlineAllocator<UE::Mass::LookAt::HashGridResultsSoftLimit>> SpanIndices;
			for (int32 ItemIndex = 0; ItemIndex < NearbyEntities.Num(); ItemIndex++)
			{
				const UE::Mass::LookAt::FTargetHashGridItem& Item = NearbyEntities[ItemIndex];
				if (!LastPriority.IsSet() || Item.Priority != LastPriority.GetValue())
				{
					LastPriority = Item.Priority;
					SpanIndices.Add(ItemIndex);
				}
			}

			// We'll pick the first entity that passes, this ensures that it's a random one.
			// For now higher priority targets are always selected first
			for (int32 SpanIndex = 0; SpanIndex < SpanIndices.Num(); SpanIndex++)
			{
				const int32 FirstItemIndex = SpanIndices[SpanIndex];
				const int32 NextSpanItemIndex = SpanIndex + 1 < SpanIndices.Num() ? SpanIndices[SpanIndex + 1] : NearbyEntities.Num();
				const int32 NumItems = NextSpanItemIndex - FirstItemIndex;
				Algo::RandomShuffle(MakeArrayView(&NearbyEntities[FirstItemIndex], NumItems));
			}

			const FVector Location = Transform.GetLocation();
			for (const UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType& Item : NearbyEntities)
			{
				FMassEntityHandle CandidateEntity = Item.TargetEntity;

				// This can happen if we remove entities in the system.
				if (!EntityManager.IsEntityValid(CandidateEntity))
				{
					UE_LOGF(LogMassBehavior, VeryVerbose, "Nearby entity is invalid, skipped.");
					continue;
				}

				// Do not select self
				if (CandidateEntity == Entity)
				{
					continue;
				}

				// Do not select same target twice in a row.
				if (CandidateEntity == LastTrackedEntity)
				{
					continue;
				}

				// TargetFragment and Transform are added through the LookAtTargetTrait
				const FTransformFragment& TargetTransform = EntityManager.GetFragmentDataChecked<FTransformFragment>(CandidateEntity);
				const FMassLookAtTargetFragment& TargetFragment = EntityManager.GetFragmentDataChecked<FMassLookAtTargetFragment>(CandidateEntity);

				const FVector TargetLocation = TargetTransform.GetTransform().TransformPosition(TargetFragment.Offset);
				if (!QueryBox.IsInside(TargetLocation))
				{
					continue;
				}

				FVector Direction = (TargetLocation - Location).GetSafeNormal();
				Direction = Transform.InverseTransformVector(Direction);

				const bool bIsTargetInView = FVector::DotProduct(FVector::ForwardVector, Direction) > CosAngleThreshold;
				if (bIsTargetInView)
				{
					LookAt.GazeDirection = Direction;
					LookAt.GazeTrackedEntity = CandidateEntity;
					LookAt.GazeTargetLocation = TargetLocation;
					bTargetFound = true;
					break;
				}

				// Allow to pick entities out of view if they are moving towards us.
				if (const FMassVelocityFragment* Velocity = EntityManager.GetFragmentDataPtr<FMassVelocityFragment>(CandidateEntity))
				{
					const FVector MoveDirection = Transform.InverseTransformVector(Velocity->Value.GetSafeNormal());

					// Direction negated as it is from the agent to target, and we want target to agent.
					const bool bIsTargetMovingTowards = FVector::DotProduct(MoveDirection, -Direction) > CosAngleThreshold;
					if (bIsTargetMovingTowards)
					{
						LookAt.GazeDirection = Direction;
						LookAt.GazeTrackedEntity = CandidateEntity;
						LookAt.GazeTargetLocation = TargetLocation;
						bTargetFound = true;
						break;
					}
				}
			}
		}
	}

	// If no gaze target found, use random angle if specified.
	if (!bTargetFound)
	{
		const FRotator Rot(FMath::FRandRange(-(float)LookAt.RandomGazePitchVariation, (float)LookAt.RandomGazePitchVariation),FMath::FRandRange(-(float)LookAt.RandomGazeYawVariation, (float)LookAt.RandomGazeYawVariation), 0.f);
		LookAt.GazeDirection = UE::MassBehavior::ClampDirectionToXAxisCone(Rot.Vector(), FMath::DegreesToRadians(AngleThresholdInDegrees));
		LookAt.GazeTargetLocation = FVector::ZeroVector;
	}

	// @todo LookAt: This does not currently carry over time. It's intentional, since there might be big gaps between updates.
	LookAt.GazeStartTime = CurrentTime;
	LookAt.GazeDuration = FMath::FRandRange(FMath::Max(Duration - DurationVariation, 0.f), Duration + DurationVariation);
}

void UMassLookAtProcessor::UpdateLookAtTrajectory(const FTransform& Transform, const FMassZoneGraphLaneLocationFragment& ZoneGraphLocation,
												  const FMassLookAtTrajectoryFragment& LookAtTrajectory, const bool bDisplayDebug, FMassLookAtFragment& LookAt) const
{
	if (LookAtTrajectory.NumPoints > 0 && LookAtTrajectory.LaneHandle == ZoneGraphLocation.LaneHandle)
	{
		// Look at anticipated position in the future when moving.
		const float LookAheadDistanceAlongPath = ZoneGraphLocation.DistanceAlongLane + UE::MassBehavior::Tweakables::TrajectoryLookAhead * (LookAtTrajectory.bMoveReverse ? -1.0f : 1.0f);

		// Calculate lookat direction to the anticipated position.
		const FVector AnticipatedPosition = LookAtTrajectory.GetPointAtDistanceExtrapolated(LookAheadDistanceAlongPath);
		const FVector AgentPosition = Transform.GetLocation();
		const FVector NewGlobalDirection = (AnticipatedPosition - AgentPosition).GetSafeNormal();
		LookAt.Direction = Transform.InverseTransformVector(NewGlobalDirection);
		LookAt.Direction.Z = 0.0f;
		LookAt.MainTargetLocation = AnticipatedPosition;

#if WITH_MASSGAMEPLAY_DEBUG
		if (bDisplayDebug)
		{
			const FVector ZOffset(0.f,0.f,DebugZOffset);
			UE_VLOG_SEGMENT_THICK(this, LogMassBehavior, Display, AgentPosition + ZOffset, AgentPosition + ZOffset + 100.f * NewGlobalDirection, FColor::White, /*Thickness*/3, TEXT("LookAt Trajectory"));
		}
#endif
	}
}

void UMassLookAtProcessor::UpdateLookAtTrackedEntity(const FMassEntityManager& EntityManager, const FTransform& Transform, const bool bDisplayDebug, FMassLookAtFragment& LookAt) const
{
	// Update direction toward target
	if (EntityManager.IsEntityValid(LookAt.TrackedEntity))
	{
		const FTransformFragment& TargetTransform = EntityManager.GetFragmentDataChecked<FTransformFragment>(LookAt.TrackedEntity);
		const FMassLookAtTargetFragment* TargetFragment = EntityManager.GetFragmentDataPtr<FMassLookAtTargetFragment>(LookAt.TrackedEntity);

		const FVector AgentPosition = Transform.GetLocation();
		const FVector TargetBaseLocation = TargetTransform.GetTransform().GetLocation();
		const FVector NewGlobalDirection = (TargetBaseLocation - AgentPosition).GetSafeNormal();
		LookAt.Direction = Transform.InverseTransformVector(NewGlobalDirection);
		LookAt.MainTargetLocation = TargetFragment != nullptr ? TargetTransform.GetTransform().TransformPosition(TargetFragment->Offset) : TargetBaseLocation;

#if WITH_MASSGAMEPLAY_DEBUG
		if (bDisplayDebug)
		{
			const FVector ZOffset(0.f,0.f,DebugZOffset);
			UE_VLOG_SEGMENT_THICK(this, LogMassBehavior, Display, AgentPosition + ZOffset, AgentPosition + ZOffset + 100.f * NewGlobalDirection, FColor::White, /*Thickness*/3, TEXT("LookAt Track"));
		}
#endif
	}
}

bool UMassLookAtProcessor::UpdateGazeTrackedEntity(const FMassEntityManager& EntityManager, const FTransform& Transform, const bool bDisplayDebug, FMassLookAtFragment& LookAt) const
{
	bool bHasTarget = false;

	// Update direction toward gaze target
	if (LookAt.GazeTrackedEntity.IsSet() && EntityManager.IsEntityValid(LookAt.GazeTrackedEntity))
	{
		const FTransformFragment& TargetTransform = EntityManager.GetFragmentDataChecked<FTransformFragment>(LookAt.GazeTrackedEntity);
		const FMassLookAtTargetFragment* TargetFragment = EntityManager.GetFragmentDataPtr<FMassLookAtTargetFragment>(LookAt.GazeTrackedEntity);

		const FVector AgentPosition = Transform.GetLocation();
		const FVector TargetBaseLocation = TargetTransform.GetTransform().GetLocation();
		const FVector NewGlobalDirection = (TargetBaseLocation - AgentPosition).GetSafeNormal();
		LookAt.GazeDirection = Transform.InverseTransformVector(NewGlobalDirection);
		LookAt.GazeTargetLocation = TargetFragment != nullptr ? TargetTransform.GetTransform().TransformPosition(TargetFragment->Offset) : TargetBaseLocation;

		bHasTarget = true;

#if WITH_MASSGAMEPLAY_DEBUG
		if (bDisplayDebug)
		{
			const FVector ZOffset(0.f, 0.f, DebugZOffset);
			UE_VLOG_SEGMENT_THICK(this, LogMassBehavior, Display, AgentPosition + ZOffset, AgentPosition + ZOffset + 100.f * NewGlobalDirection, FColor(160, 160, 160), /*Thickness*/3, TEXT("Gaze Track"));
		}
#endif
	}
	return bHasTarget;
}

void UMassLookAtProcessor::BuildTrajectory(const UZoneGraphSubsystem& ZoneGraphSubsystem, const FMassZoneGraphLaneLocationFragment& LaneLocation, const FMassZoneGraphShortPathFragment& ShortPath,
											const FMassEntityHandle Entity, const bool bDisplayDebug, FMassLookAtTrajectoryFragment& LookAtTrajectory)
{
	LookAtTrajectory.Reset();

	if (ShortPath.NumPoints < 2)
	{
		return;
	}

	LookAtTrajectory.bMoveReverse = ShortPath.bMoveReverse;
	LookAtTrajectory.LaneHandle = LaneLocation.LaneHandle;

	const float NextLaneLookAheadDistance = UE::MassBehavior::Tweakables::TrajectoryLookAhead;
	
	// Initialize the look at trajectory from the current path.
	const FMassZoneGraphPathPoint& FirstPathPoint = ShortPath.Points[0];
	const FMassZoneGraphPathPoint& LastPathPoint = ShortPath.Points[ShortPath.NumPoints - 1];
	ensure(LookAtTrajectory.AddPoint(FirstPathPoint.Position, FirstPathPoint.Tangent.Get(), FirstPathPoint.DistanceAlongLane.Get()));
	ensure(LookAtTrajectory.AddPoint(LastPathPoint.Position, LastPathPoint.Tangent.Get(), LastPathPoint.DistanceAlongLane.Get()));

	// If the path will lead to next lane, add a point from next lane too.
	if (ShortPath.NextLaneHandle.IsValid())
	{
		const FZoneGraphStorage* ZoneGraphStorage = ZoneGraphSubsystem.GetZoneGraphStorage(LaneLocation.LaneHandle.DataHandle);
		if (ZoneGraphStorage != nullptr)
		{
			if (ShortPath.NextExitLinkType == EZoneLaneLinkType::Outgoing)
			{
				FZoneGraphLaneLocation Location;
				UE::ZoneGraph::Query::CalculateLocationAlongLane(*ZoneGraphStorage, ShortPath.NextLaneHandle, NextLaneLookAheadDistance, Location);

				ensure(LookAtTrajectory.AddPoint(Location.Position, FVector2D(Location.Tangent), LastPathPoint.DistanceAlongLane.Get() + Location.DistanceAlongLane));
			}
			else if (ShortPath.NextExitLinkType == EZoneLaneLinkType::Incoming)
			{
				float LaneLength = 0.0f;
				UE::ZoneGraph::Query::GetLaneLength(*ZoneGraphStorage, ShortPath.NextLaneHandle, LaneLength);

				FZoneGraphLaneLocation Location;
				UE::ZoneGraph::Query::CalculateLocationAlongLane(*ZoneGraphStorage, ShortPath.NextLaneHandle, LaneLength - NextLaneLookAheadDistance, Location);

				// Moving backwards, reverse tangent and distance.
				ensure(LookAtTrajectory.bMoveReverse);
				ensure(LookAtTrajectory.AddPoint(Location.Position, FVector2D(-Location.Tangent), LastPathPoint.DistanceAlongLane.Get() - (LaneLength - Location.DistanceAlongLane)));
			}
			else if (ShortPath.NextExitLinkType == EZoneLaneLinkType::Adjacent)
			{
				// No extra point
			}
			else
			{
				ensureMsgf(false, TEXT("Unhandle NextExitLinkType type %s"), *UEnum::GetValueAsString(ShortPath.NextExitLinkType));
			}
		}
		else
		{
			UE_CVLOG(bDisplayDebug, this, LogMassBehavior, Error, TEXT("%s Could not find ZoneGraph storage for lane %s."),
				*Entity.DebugGetDescription(), *LaneLocation.LaneHandle.ToString());
		}				
	}

	// Ensure that the points are always in ascending distance order (it is, in case of reverse path).
	if (LookAtTrajectory.NumPoints > 1 && LookAtTrajectory.bMoveReverse)
	{
		ensureMsgf(LookAtTrajectory.Points[0].DistanceAlongLane.Get() >= LookAtTrajectory.Points[LookAtTrajectory.NumPoints - 1].DistanceAlongLane.Get(),
			TEXT("Expecting trajectory 0 (%.1f) >= %d (%.1f)"), LookAtTrajectory.Points[0].DistanceAlongLane.Get(),
			LookAtTrajectory.NumPoints - 1, LookAtTrajectory.Points[LookAtTrajectory.NumPoints - 1].DistanceAlongLane.Get());
		
		Algo::Reverse(LookAtTrajectory.Points.GetData(), LookAtTrajectory.NumPoints);
		// Tangents needs to be reversed when the trajectory is reversed.
		for (uint8 PointIndex = 0; PointIndex < LookAtTrajectory.NumPoints; PointIndex++)
		{
			LookAtTrajectory.Points[PointIndex].Tangent.Set(-LookAtTrajectory.Points[PointIndex].Tangent.Get());
		}
	}

#if WITH_MASSGAMEPLAY_DEBUG
	if (bDisplayDebug)
	{
		const FVector ZOffset(0,0,35);
		
		for (uint8 PointIndex = 0; PointIndex < LookAtTrajectory.NumPoints - 1; PointIndex++)
		{
			const FMassLookAtTrajectoryPoint& CurrPoint = LookAtTrajectory.Points[PointIndex];
			const FMassLookAtTrajectoryPoint& NextPoint = LookAtTrajectory.Points[PointIndex + 1];

			// Trajectory
			const FVector StartPoint = CurrPoint.Position;
			const FVector StartForward = CurrPoint.Tangent.GetVector();
			const FVector EndPoint = NextPoint.Position;
			const FVector EndForward = NextPoint.Tangent.GetVector();
			const FVector::FReal TangentDistance = FVector::Dist(StartPoint, EndPoint) / 3.;
			const FVector StartControlPoint = StartPoint + StartForward * TangentDistance;
			const FVector EndControlPoint = EndPoint - EndForward * TangentDistance;

			static constexpr int32 NumTicks = 6;
			static constexpr float DeltaT = 1.0f / NumTicks;
			
			FVector PrevPoint = StartPoint;
			for (int32 j = 0; j < NumTicks; j++)
			{
				const float T = static_cast<float>(j + 1) * DeltaT;
				const FVector Point = UE::CubicBezier::Eval(StartPoint, StartControlPoint, EndControlPoint, EndPoint, T);
				UE_VLOG_SEGMENT_THICK(this, LogMassBehavior, Display, PrevPoint + ZOffset, Point + ZOffset, FColor::White, /*Thickness*/3, TEXT(""));
				PrevPoint = Point;
			}
		}
		
		for (uint8 PointIndex = 0; PointIndex < LookAtTrajectory.NumPoints; PointIndex++)
		{
			const FMassLookAtTrajectoryPoint& CurrPoint = LookAtTrajectory.Points[PointIndex];
			const FVector CurrBase = CurrPoint.Position + ZOffset * 1.1f;
			// Tangents
			UE_VLOG_SEGMENT_THICK(this, LogMassBehavior, Display, CurrBase, CurrBase + CurrPoint.Tangent.GetVector() * 100.0f, FColorList::Grey, /*Thickness*/1,
				TEXT("D:%.1f"), CurrPoint.DistanceAlongLane.Get());
		}
	}
#endif // WITH_MASSGAMEPLAY_DEBUG
}

//----------------------------------------------------------------------//
// UMassLookAtTargetProcessor
//----------------------------------------------------------------------//
UMassLookAtTargetGridProcessor::UMassLookAtTargetGridProcessor()
	: AddToGridQuery(*this)
	, UpdateGridQuery(*this)
	, RemoveFromGridQuery(*this)
{
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllNetModes);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Tasks);
}

void UMassLookAtTargetGridProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	FMassEntityQuery BaseQuery(EntityManager);
	BaseQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	BaseQuery.AddRequirement<FMassLookAtTargetFragment>(EMassFragmentAccess::ReadWrite);
	BaseQuery.AddSubsystemRequirement<UMassLookAtSubsystem>(EMassFragmentAccess::ReadWrite);

	AddToGridQuery = BaseQuery;
	AddToGridQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	AddToGridQuery.AddTagRequirement<FMassInLookAtTargetGridTag>(EMassFragmentPresence::None);
	AddToGridQuery.RegisterWithProcessor(*this);

	UpdateGridQuery = BaseQuery;
	UpdateGridQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	UpdateGridQuery.AddTagRequirement<FMassInLookAtTargetGridTag>(EMassFragmentPresence::All);
	UpdateGridQuery.RegisterWithProcessor(*this);

	RemoveFromGridQuery = BaseQuery;
	RemoveFromGridQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	RemoveFromGridQuery.AddTagRequirement<FMassInLookAtTargetGridTag>(EMassFragmentPresence::All);
	RemoveFromGridQuery.RegisterWithProcessor(*this);
}

void UMassLookAtTargetGridProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(LookAtProcessor_Run);

	constexpr float Radius = 50.f;

	AddToGridQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassLookAtSubsystem& LookAtSubsystem = Context.GetMutableSubsystemChecked<UMassLookAtSubsystem>();
		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		const TArrayView<FMassLookAtTargetFragment> TargetList = Context.GetMutableFragmentView<FMassLookAtTargetFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassLookAtTargetFragment& Target = TargetList[EntityIt];
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			const FVector NewPos = LocationList[EntityIt].GetTransform().GetLocation();
			const FBox NewBounds(NewPos - FVector(Radius, Radius, 0.f), NewPos + FVector(Radius, Radius, 0.f));
			Target.CellLocation = LookAtSubsystem.AddTarget(Entity, Target, NewBounds);

			Context.Defer().AddTag<FMassInLookAtTargetGridTag>(Entity);
		}
	});

	UpdateGridQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassLookAtSubsystem& LookAtSubsystem = Context.GetMutableSubsystemChecked<UMassLookAtSubsystem>();
		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		const TArrayView<FMassLookAtTargetFragment> CellLocationList = Context.GetMutableFragmentView<FMassLookAtTargetFragment>();
		TArray<TTuple<const FMassEntityHandle, FMassLookAtTargetFragment&, const FBox>> AllUpdates;

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FVector NewPos = LocationList[EntityIt].GetTransform().GetLocation();
			const FBox NewBounds(NewPos - FVector(Radius, Radius, 0.f), NewPos + FVector(Radius, Radius, 0.f));
			AllUpdates.Emplace(Context.GetEntity(EntityIt), CellLocationList[EntityIt], NewBounds);
		}

		LookAtSubsystem.BatchMoveTarget(AllUpdates);
	});

	RemoveFromGridQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		UMassLookAtSubsystem& LookAtSubsystem = Context.GetMutableSubsystemChecked<UMassLookAtSubsystem>();
		const TArrayView<FMassLookAtTargetFragment> TargetList = Context.GetMutableFragmentView<FMassLookAtTargetFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			LookAtSubsystem.RemoveTarget(Entity, TargetList[EntityIt]);
			TargetList[EntityIt].CellLocation = UE::Mass::LookAt::FTargetHashGrid2D::FCellLocation();

			Context.Defer().RemoveTag<FMassInLookAtTargetGridTag>(Entity);
		}
	});
}

//----------------------------------------------------------------------//
//  UMassLookAtTargetRemoverProcessor
//----------------------------------------------------------------------//
UMassLookAtTargetRemoverProcessor::UMassLookAtTargetRemoverProcessor()
	: Query(*this)
{
	ObservedTypes.Add(FMassLookAtTargetFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::AllNetModes);
}

void UMassLookAtTargetRemoverProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Query.AddRequirement<FMassLookAtTargetFragment>(EMassFragmentAccess::ReadWrite);
	Query.AddSubsystemRequirement<UMassLookAtSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassLookAtTargetRemoverProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	Query.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UMassLookAtSubsystem& LookAtSubsystem = Context.GetMutableSubsystemChecked<UMassLookAtSubsystem>();
		const TArrayView<FMassLookAtTargetFragment> TargetList = Context.GetMutableFragmentView<FMassLookAtTargetFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			LookAtSubsystem.RemoveTarget(Context.GetEntity(EntityIt), TargetList[EntityIt]);
		}
	});
}

//----------------------------------------------------------------------//
//  UMassLookAtRequestInitializer
//----------------------------------------------------------------------//
UMassLookAtRequestInitializer::UMassLookAtRequestInitializer()
	: Query(*this)
{
	ObservedTypes.Add(FMassLookAtRequestFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);
}

void UMassLookAtRequestInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Query.AddRequirement<FMassLookAtRequestFragment>(EMassFragmentAccess::ReadOnly);
	ProcessorRequirements.AddSubsystemRequirement<UMassLookAtSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassLookAtRequestInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	UMassLookAtSubsystem& LookAtSubsystem = Context.GetMutableSubsystemChecked<UMassLookAtSubsystem>();
	TArray<UMassLookAtSubsystem::FRequest> Requests;

	Query.ForEachEntityChunk(Context, [&Requests](FMassExecutionContext& Context)
	{
		const TConstArrayView<FMassLookAtRequestFragment> RequestList = Context.GetFragmentView<FMassLookAtRequestFragment>();

		Requests.Reserve(Requests.Num() + Context.GetNumEntities());
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			Requests.Add({ Context.GetEntity(EntityIt), RequestList[EntityIt] });
		}
	});

	LookAtSubsystem.RegisterRequests(Context, MoveTemp(Requests));
}

//----------------------------------------------------------------------//
//  UMassLookAtRequestInitializer
//----------------------------------------------------------------------//
UMassLookAtRequestDeinitializer::UMassLookAtRequestDeinitializer()
	: Query(*this)
{
	ObservedTypes.Add(FMassLookAtRequestFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = static_cast<int32>(EProcessorExecutionFlags::Client | EProcessorExecutionFlags::Standalone);
}

void UMassLookAtRequestDeinitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Query.AddRequirement<FMassLookAtRequestFragment>(EMassFragmentAccess::ReadOnly);
	ProcessorRequirements.AddSubsystemRequirement<UMassLookAtSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassLookAtRequestDeinitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	UMassLookAtSubsystem& LookAtSubsystem = Context.GetMutableSubsystemChecked<UMassLookAtSubsystem>();
	TArray<FMassEntityHandle> Requests;

	Query.ForEachEntityChunk(Context, [&Requests](FMassExecutionContext& Context)
	{
		Requests.Append(Context.GetEntities());
	});

	LookAtSubsystem.UnregisterRequests(Context, Requests);
}
