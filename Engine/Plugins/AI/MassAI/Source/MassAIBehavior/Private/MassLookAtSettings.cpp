// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtSettings.h"

//----------------------------------------------------------------------//
// UMassLookAtSettings
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLookAtSettings)
FOnMassLookAtPrioritiesChanged UMassLookAtSettings::OnMassLookAtPrioritiesChanged;

UMassLookAtSettings::UMassLookAtSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	// Setup default values, the config file will override these when the user changes them.
	for (int32 PriorityIndex = 0; PriorityIndex < static_cast<int32>(EMassLookAtPriorities::MaxPriorities); PriorityIndex++)
	{
		FMassLookAtPriorityInfo& Info = Priorities[PriorityIndex];
		Info.Priority = FMassLookAtPriority(static_cast<uint8>(PriorityIndex));
	}

	constexpr int32 LowestPriorityIndex = static_cast<int32>(EMassLookAtPriorities::LowestPriority);
	Priorities[0].Name = FName(TEXT("Highest"));
	Priorities[LowestPriorityIndex].Name = FName(TEXT("Lowest (default)"));
}

#if WITH_EDITOR
void UMassLookAtSettings::PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent)
{
	Super::PostEditChangeChainProperty(PropertyChangedEvent);

	const FProperty* MemberProperty = nullptr;
	if (PropertyChangedEvent.PropertyChain.GetActiveMemberNode())
	{
		MemberProperty = PropertyChangedEvent.PropertyChain.GetActiveMemberNode()->GetValue();
	}

	if (MemberProperty
		&& MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMassLookAtSettings, Priorities))
	{
		OnMassLookAtPrioritiesChanged.Broadcast();
	}
}
#endif // WITH_EDITOR

void UMassLookAtSettings::GetValidPriorityInfos(TArray<FMassLookAtPriorityInfo>& OutInfos) const
{
	for (int32 i = 0; i < static_cast<int32>(EMassLookAtPriorities::MaxPriorities); i++)
	{
		const FMassLookAtPriorityInfo& Info = Priorities[i];
		if (Info.IsValid())
		{
			OutInfos.Add(Info);
		}
	}
}
