// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtSubsystem.h"
#include "GameFramework/Pawn.h"
#include "MassActorSubsystem.h"
#include "MassAIBehaviorTypes.h"
#include "MassEntitySubsystem.h"
#include "MassEntityView.h"
#include "MassSignalSubsystem.h"
#include "MassStateTreeTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLookAtSubsystem)

namespace UE::Mass::LookAt::Private
{
void FlushCommands(FMassEntityManager& EntityManager, const TSharedPtr<FMassCommandBuffer>& CommandBuffer)
{
	ExecuteOnGameThread(UE_SOURCE_LOCATION, [WeakEntityManager = EntityManager.AsWeak(), CommandBuffer]()
		{
			if (const TSharedPtr<FMassEntityManager> SharedEntityManager = WeakEntityManager.Pin())
			{
				SharedEntityManager->FlushCommands(CommandBuffer);
			}
			else
			{
				CommandBuffer->CancelCommands();
			}
		});
}

bool CreateRequests(FMassEntityManager& InEntityManager
	, const FMassArchetypeHandle& RequestArchetype
	, const TConstArrayView<FMassEntityHandle> Viewers
	, const TConstArrayView<FMassEntityHandle> Requests
	, const FMassEntityHandle TargetEntity
	, const FMassLookAtPriority Priority
	, const EMassLookAtInterpolationSpeed InterpolationSpeed
	, const float CustomInterpolationSpeed = DefaultCustomInterpolationSpeed)
{
	if (Viewers.IsEmpty())
	{
		return false;
	}

	checkf(Viewers.Num() == Requests.Num(), TEXT("Number of reserved entities for requests must match the number of provided viewer entities."));

	const TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext = InEntityManager.BatchCreateReservedEntities(RequestArchetype, /*SharedFragmentValues*/{}, Requests);

	for (int Index = 0; Index < Requests.Num(); ++Index)
	{
		InEntityManager.SetEntityFragmentValues(Requests[Index], { FInstancedStruct::Make(
			FMassLookAtRequestFragment
			{
				Viewers[Index]
				, Priority
				, EMassLookAtMode::LookAtEntity
				, TargetEntity
				, InterpolationSpeed
				, CustomInterpolationSpeed
			})});

		UE_VLOG_UELOG(InEntityManager.GetOwner(), LogMassBehavior, Log, TEXT("Created LookAt Request '%s', Target '%s', Priority=%u")
			, *LexToString(Requests[Index])
			, *LexToString(TargetEntity)
			, Priority.Get());
	}

	return true;
}

bool CreateTargets(FMassEntityManager& InEntityManager
	, const FMassArchetypeHandle& TargetArchetype
	, const TConstArrayView<FMassEntityHandle> Targets
	, const TConstArrayView<const FTransform> Transforms
	, const FMassLookAtPriority Priority)
{
	if (Targets.IsEmpty())
	{
		return false;
	}

	checkf(Targets.Num() == Transforms.Num(), TEXT("Number of reserved entities for targets must match the number of provided transforms."));

	// This needs to stay in sync with 'TargetArchetype' created on subsystem initialization
	TArray<FInstancedStruct, TInlineAllocator<2>> FragmentInstanceList;
	FragmentInstanceList.Add(FInstancedStruct::Make(FMassLookAtTargetFragment{ .Offset = FVector::ZeroVector, .Priority = Priority }));
	FTransformFragment& TransformFragment = FragmentInstanceList.Add_GetRef({ FInstancedStruct::Make(FTransformFragment{})}).GetMutable<FTransformFragment>();

	const TSharedRef<FMassEntityManager::FEntityCreationContext> CreationContext =
		InEntityManager.BatchCreateReservedEntities(TargetArchetype, /*SharedFragmentValues*/{}, Targets);

	for (int Index = 0; Index < Targets.Num(); ++Index)
	{
		TransformFragment.SetTransform(Transforms[Index]);
		InEntityManager.SetEntityFragmentValues(Targets[Index], FragmentInstanceList);

		UE_VLOG_UELOG(InEntityManager.GetOwner(), LogMassBehavior, Log, TEXT("Created LookAtTarget '%s' at '%s'")
			, *LexToString(Targets[Index])
			, *LexToString(TransformFragment.GetTransform().ToString()))
		UE_VLOG_LOCATION(InEntityManager.GetOwner(), LogMassBehavior, Display
			, TransformFragment.GetTransform().GetLocation(), /*Thickness*/50, FColor::Yellow, TEXT(""));
	}

	return true;
}

TArray<FMassEntityHandle> DebugRequests;

static FAutoConsoleCommandWithWorldArgsAndOutputDevice CmdSendLookAtPlayerRequestToAll(
	TEXT("ai.debug.mass.SendLookAtPlayerRequestToAll"),
	TEXT("Creates, or removed, LookAt requests toward the player for all mass entities with a LookAt fragment"
		"(optional)<0|1> to indicate if the requests must be created (1, default) or deleted (0)."
		"(optional)<int> to indicate the priority of the request where a lower value represents a higher priority (default is 5)"),
	FConsoleCommandWithWorldArgsAndOutputDeviceDelegate::CreateLambda([](const TArray<FString>& Args, const UWorld* World, FOutputDevice& OutputDevice)
		{
			bool bCreateRequest = true;

			uint8 PriorityLevel = static_cast<uint8>(EMassLookAtPriorities::LowestPriority);

			if (Args.Num() > 2)
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed: invalid number of arguments"));
				return;
			}

			if (Args.Num() > 0)
			{
				if (!LexTryParseString(bCreateRequest, *Args[0]))
				{
					OutputDevice.Log(ELogVerbosity::Error, TEXT("Unable to parse the first argument: expecting 0|1 or true|false"));
					return;
				}

				if (Args.Num() > 1)
				{
					if (!LexTryParseString(PriorityLevel, *Args[1]))
					{
						OutputDevice.Log(ELogVerbosity::Error, TEXT("Unable to parse the second argument: expecting an [0-255] integer to represent the priority"));
						return;
					}

					// Clamp priority in valid range (highest value being the lowest priority)
					if (constexpr uint8 LastPriority = static_cast<uint8>(EMassLookAtPriorities::LowestPriority); PriorityLevel > LastPriority)
					{
						OutputDevice.Log(ELogVerbosity::Warning, FString::Printf(TEXT("Clamped priority level to the lowest priority %d"), LastPriority));
						PriorityLevel = FMath::Clamp<uint8>(PriorityLevel, 0, LastPriority);
					}
				}
			}

			UMassActorSubsystem* ActorSubsystem = World->GetSubsystem<UMassActorSubsystem>();
			if (ActorSubsystem == nullptr)
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed: unable to find MassActorSubsystem"));
				return;
			}

			const APlayerController* PlayerController = World->GetFirstPlayerController();
			if (PlayerController == nullptr)
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed: unable to find the player controller"));
				return;
			}

			const APawn* PlayerPawn = PlayerController->GetPawn();
			if (PlayerPawn == nullptr)
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed: unable to find the player pawn"));
				return;
			}

			UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
			if (EntitySubsystem == nullptr)
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed: unable to find UMassEntitySubsystem"));
				return;
			}

			const FMassEntityHandle PlayerEntity = ActorSubsystem->GetEntityHandleFromActor(PlayerPawn);
			if (!PlayerEntity.IsSet())
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed: unable to find a MassEntity associated to the player"));
				return;
			}

			const UMassLookAtSubsystem* LookAtSubsystem = World->GetSubsystem<UMassLookAtSubsystem>();
			if (LookAtSubsystem == nullptr)
			{
				OutputDevice.Log(ELogVerbosity::Error, TEXT("Command failed: unable to find UMassLookAtSubsystem"));
				return;
			}

			TSharedPtr<FMassCommandBuffer> CommandBuffer = MakeShareable(new FMassCommandBuffer());
			if (bCreateRequest)
			{
				// Create requests for all entities with a MassLookAtFragment
				CommandBuffer->PushCommand<FMassDeferredCreateCommand>([PlayerEntity, PriorityLevel, RequestArchetype = LookAtSubsystem->DebugGetRequestArchetype()](FMassEntityManager& InEntityManager)
					{
						FMassEntityQuery EntityQuery(InEntityManager.AsShared());
						EntityQuery.AddRequirement<FMassLookAtFragment>(EMassFragmentAccess::ReadOnly);
						TArray<FMassEntityHandle> Viewers = EntityQuery.GetMatchingEntityHandles();

						CreateRequests(InEntityManager
							, RequestArchetype
							, Viewers
							, InEntityManager.BatchReserveEntities(Viewers.Num(), DebugRequests)
							, PlayerEntity
							, FMassLookAtPriority{ PriorityLevel }
							, EMassLookAtInterpolationSpeed::Regular);
					});
			}
			else
			{
				// Delete all entities created for debug requests
				CommandBuffer->PushCommand<FMassDeferredDestroyCommand>([](FMassEntityManager& InEntityManager)
					{
						InEntityManager.BatchDestroyEntities(DebugRequests);
						DebugRequests.Reset();
					});
			}

			FlushCommands(EntitySubsystem->GetMutableEntityManager(), CommandBuffer);
		}
	));

} // UE::Mass::LookAt::Private

FMassLookAtRequestHandle UMassLookAtSubsystem::CreateLookAtPositionRequest(
	AActor* ViewerActor
	, const FMassLookAtPriority Priority
	, const FVector TargetLocation
	, const EMassLookAtInterpolationSpeed InterpolationSpeed
	, const float CustomInterpolationSpeed) const
{
	const UWorld* World = GetWorld();

	UMassActorSubsystem* ActorSubsystem = World->GetSubsystem<UMassActorSubsystem>();
	if (ActorSubsystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find MassActorSubsystem"), __FUNCTION__);
		return {};
	}

	UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
	if (EntitySubsystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find UMassEntitySubsystem"), __FUNCTION__);
		return {};
	}

	const FMassEntityHandle ViewerEntity = ActorSubsystem->GetEntityHandleFromActor(ViewerActor);
	if (!ViewerEntity.IsSet())
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find a MassEntity associated to '%s'"), __FUNCTION__, *GetNameSafe(ViewerActor));
		return {};
	}

	// @todo: once available, consider using incoming EntityBuilder improvements for this whole creation step
	FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();
	FMassEntityHandle TargetEntity = EntityManager.ReserveEntity();
	FMassEntityHandle RequestEntity = EntityManager.ReserveEntity();
	FTransform TargetTransform(TargetLocation);

	// Push command to create a new entity representing a LookAt target along with a request to look at it
	const TSharedPtr<FMassCommandBuffer> CommandBuffer = MakeShareable(new FMassCommandBuffer());
	CommandBuffer->PushCommand<FMassDeferredCreateCommand>(
		[
			TargetArchetype = TargetArchetype
			, RequestArchetype = RequestArchetype
			, RequestEntity
			, ViewerEntity
			, TargetEntity
			, TargetTransform
			, Priority
			, InterpolationSpeed
			, CustomInterpolationSpeed
		]
		(FMassEntityManager& InEntityManager)
		{
			UE::Mass::LookAt::Private::CreateTargets(InEntityManager, TargetArchetype, { TargetEntity }, { TargetTransform }, Priority);

			UE::Mass::LookAt::Private::CreateRequests(
				InEntityManager
				, RequestArchetype
				, { ViewerEntity }
				, { RequestEntity }
				, TargetEntity
				, Priority
				, InterpolationSpeed
				, CustomInterpolationSpeed);
		});

	UE::Mass::LookAt::Private::FlushCommands(EntityManager, CommandBuffer);

	return FMassLookAtRequestHandle{RequestEntity, TargetEntity, /*bTargetEntityOwnedByRequest*/true};
}

FMassLookAtRequestHandle UMassLookAtSubsystem::CreateLookAtActorRequest(
	AActor* ViewerActor
	, const FMassLookAtPriority Priority
	, AActor* TargetActor
	, const EMassLookAtInterpolationSpeed InterpolationSpeed
	, const float CustomInterpolationSpeed) const
{
	const UWorld* World = GetWorld();

	if (TargetActor == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Log, TEXT("%hs failed: invalid target actor"), __FUNCTION__);
		return {};
	}

	UMassActorSubsystem* ActorSubsystem = World->GetSubsystem<UMassActorSubsystem>();
	if (ActorSubsystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find MassActorSubsystem"), __FUNCTION__);
		return {};
	}

	const FMassEntityHandle TargetEntity = ActorSubsystem->GetEntityHandleFromActor(TargetActor);
	if (!TargetEntity.IsSet())
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Log, TEXT("%hs: using static target location since no MassEntity is associated to '%s'"), __FUNCTION__, *GetNameSafe(TargetActor));
		return CreateLookAtPositionRequest(ViewerActor, Priority, TargetActor->GetActorLocation(), InterpolationSpeed, CustomInterpolationSpeed);
	}

	UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
	if (EntitySubsystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find UMassEntitySubsystem"), __FUNCTION__);
		return {};
	}

	const FMassEntityHandle ViewerEntity = ActorSubsystem->GetEntityHandleFromActor(ViewerActor);
	if (!ViewerEntity.IsSet())
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find a MassEntity associated to '%s'"), __FUNCTION__, *GetNameSafe(ViewerActor));
		return {};
	}

	// @todo: once available, consider using incoming EntityBuilder improvements for this whole creation step
	FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();
	FMassEntityHandle RequestEntity = EntityManager.ReserveEntity();

	// Push command to create a new entity representing a LookAt target along with a request to look at it
	const TSharedPtr<FMassCommandBuffer> CommandBuffer = MakeShareable(new FMassCommandBuffer());
	CommandBuffer->PushCommand<FMassDeferredCreateCommand>(
		[
			RequestArchetype = RequestArchetype
			, RequestEntity
			, ViewerEntity
			, TargetEntity
			, Priority
			, InterpolationSpeed
			, CustomInterpolationSpeed
		]
		(FMassEntityManager& InEntityManager)
		{
			UE::Mass::LookAt::Private::CreateRequests(
				InEntityManager
				, RequestArchetype
				, { ViewerEntity }
				, { RequestEntity }
				, TargetEntity
				, Priority
				, InterpolationSpeed
				, CustomInterpolationSpeed);
		});

	UE::Mass::LookAt::Private::FlushCommands(EntityManager, CommandBuffer);

	return FMassLookAtRequestHandle{RequestEntity, TargetEntity, /*bTargetEntityOwnedByRequest*/false};
}

void UMassLookAtSubsystem::DeleteRequest(const FMassLookAtRequestHandle RequestHandle) const
{
	// Simple validation when none of the entities are set since it is probably due to a bad data setup.
	// Otherwise, the EntityManager can process gracefully the handles, valid or not.
	if (!RequestHandle.Request.IsSet()
		&& !RequestHandle.Target.IsSet())
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: invalid request handle"), __FUNCTION__);
		return;
	}

	const UWorld* World = GetWorld();
	if (World == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find World"), __FUNCTION__);
		return;
	}

	UMassEntitySubsystem* EntitySubsystem = World->GetSubsystem<UMassEntitySubsystem>();
	if (EntitySubsystem == nullptr)
	{
		UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("%hs failed: unable to find UMassEntitySubsystem"), __FUNCTION__);
		return;
	}

	const TSharedPtr<FMassCommandBuffer> CommandBuffer = MakeShareable(new FMassCommandBuffer());
	CommandBuffer->PushCommand<FMassDeferredDestroyCommand>([RequestHandle](FMassEntityManager& InEntityManager)
		{
			if (RequestHandle.bTargetEntityOwnedByRequest)
			{
				InEntityManager.BatchDestroyEntities({ RequestHandle.Request, RequestHandle.Target });
			}
			else
			{
				InEntityManager.DestroyEntity(RequestHandle.Request);
			}
		});

	UE::Mass::LookAt::Private::FlushCommands(EntitySubsystem->GetMutableEntityManager(), CommandBuffer);
}

//----------------------------------------------------------------------//
//  UMassLookAtSubsystem
//----------------------------------------------------------------------//
void UMassLookAtSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	FMassEntityManager& EntityManager = Collection.InitializeDependency<UMassEntitySubsystem>()->GetMutableEntityManager();

	// Create Mass archetype for entities representing requests
	const FMassArchetypeCompositionDescriptor RequestComposition{ FMassFragmentBitSet(FMassLookAtRequestFragment::StaticStruct()) };
	RequestArchetype = EntityManager.CreateArchetype(RequestComposition, FMassArchetypeCreationParams{"LookAtRequest"});

	// Create Mass archetype for entities representing targets
	TArray<const UScriptStruct*, TInlineAllocator<2>> TargetFragmentTypes{FMassLookAtTargetFragment::StaticStruct(), FTransformFragment::StaticStruct()};
	const FMassArchetypeCompositionDescriptor TargetComposition{ FMassFragmentBitSet(TargetFragmentTypes) };
	TargetArchetype = EntityManager.CreateArchetype(TargetComposition, FMassArchetypeCreationParams{"LookAtTarget"});

	OverrideSubsystemTraits<UMassLookAtSubsystem>(Collection);
}

TStatId UMassLookAtSubsystem::GetStatId() const
{
	RETURN_QUICK_DECLARE_CYCLE_STAT(UMassLookAtSubsystem, STATGROUP_Tickables);
}

void UMassLookAtSubsystem::RegisterRequests(const FMassExecutionContext& InContext, TArray<FRequest>&& InRequests)
{
	TArray<int32> DirtyViewers;
	DirtyViewers.Reserve(InRequests.Num());

	{
		UE_MT_SCOPED_WRITE_ACCESS(RequestsAccessDetector);
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassLookAt_RegisterRequests")

		RegisteredRequests.Reserve(RegisteredRequests.Num() + (InRequests.Num() - ActiveRequestsFreeList.Num()));

		for (const FRequest& NewRequest : InRequests)
		{
			if (NewRequest.Parameters.LookAtMode == EMassLookAtMode::LookAtEntity
				&& !NewRequest.Parameters.TargetEntity.IsSet())
			{
				UE_VLOG_UELOG(this, LogMassBehavior, Error, TEXT("Ignoring LookAtEntity request: invalid target entity"));
				continue;
			}

			int32 NewRequestIndex = INDEX_NONE;
			if (ActiveRequestsFreeList.Num())
			{
				NewRequestIndex = ActiveRequestsFreeList.Pop();
				RegisteredRequests[NewRequestIndex] = NewRequest;
			}
			else
			{
				NewRequestIndex = RegisteredRequests.Add(NewRequest);
			}

			RequestHandleToIndexMap.Add(NewRequest.RequestHandle, NewRequestIndex);

			int32& ViewerDataIndex = ViewerHandleToIndexMap.FindOrAdd(NewRequest.Parameters.ViewerEntity, INDEX_NONE);
			if (ViewerDataIndex == INDEX_NONE)
			{
				ViewerDataIndex = PerViewerRequests.Num();
				PerViewerRequests.AddDefaulted_GetRef().Viewer = NewRequest.Parameters.ViewerEntity;
			}

			DirtyViewers.Add(ViewerDataIndex);
			PerViewerRequests[ViewerDataIndex].RequestIndices.Add(NewRequestIndex);
		}
	}

	UpdateLookAts(InContext, DirtyViewers);
}

void UMassLookAtSubsystem::UnregisterRequests(const FMassExecutionContext& InContext, const TConstArrayView<FMassEntityHandle> InRequests)
{
	TArray<int32> DirtyViewers;
	DirtyViewers.Reserve(InRequests.Num());

	{
		UE_MT_SCOPED_WRITE_ACCESS(RequestsAccessDetector);
		TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassLookAt_UnregisterRequests")

		for (const FMassEntityHandle& RemovedRequest : InRequests)
		{
			int32 InvalidatedIndex = INDEX_NONE;
			RequestHandleToIndexMap.RemoveAndCopyValue(RemovedRequest, InvalidatedIndex);

			if (ensureMsgf(InvalidatedIndex != INDEX_NONE, TEXT("Trying to remove a request that was never registered")))
			{
				// Invalidate entry and add to the free list
				const FMassEntityHandle ViewerEntity = RegisteredRequests[InvalidatedIndex].Parameters.ViewerEntity;
				RegisteredRequests[InvalidatedIndex].RequestHandle.Reset();
				ActiveRequestsFreeList.Push(InvalidatedIndex);

				const int32* ViewerDataIndex = ViewerHandleToIndexMap.Find(ViewerEntity);

				if (ensureMsgf(ViewerDataIndex, TEXT("Unable to find the per viewer data; looks like a bookkeeping issue")))
				{
					checkf(*ViewerDataIndex != INDEX_NONE, TEXT("INDEX_NONEs are not expected to be stored in ViewerHandleToIndexMap"));
					DirtyViewers.Add(*ViewerDataIndex);
					PerViewerRequests[*ViewerDataIndex].RequestIndices.Remove(InvalidatedIndex);
				}
			}
		}
	}

	UpdateLookAts(InContext, DirtyViewers);
}

void UMassLookAtSubsystem::UpdateLookAts(const FMassExecutionContext& Context, const TConstArrayView<int32> DirtyViewers)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("MassLookAt_UpdateLookAts")

	TArray<TPair<FMassEntityHandle, FMassLookAtRequestFragment>> Updated;
	for (const int32 DirtyViewerIndex : DirtyViewers)
	{
		check(PerViewerRequests.IsValidIndex(DirtyViewerIndex));

		int32 SelectedRequestIndex = INDEX_NONE;

		for (const int32 RequestIndex : PerViewerRequests[DirtyViewerIndex].RequestIndices)
		{
			check(RegisteredRequests.IsValidIndex(RequestIndex));
			RegisteredRequests[RequestIndex].bActive = false;

			if (SelectedRequestIndex == INDEX_NONE)
			{
				SelectedRequestIndex = RequestIndex;
				continue;
			}

			// Higher priority is represented by the lowest value
			if (RegisteredRequests[RequestIndex].Parameters.Priority.Get()
				< RegisteredRequests[SelectedRequestIndex].Parameters.Priority.Get())
			{
				SelectedRequestIndex = RequestIndex;
			}
		}

		FMassLookAtRequestFragment RequestFragment;
		if (SelectedRequestIndex != INDEX_NONE)
		{
			RegisteredRequests[SelectedRequestIndex].bActive = true;
			RequestFragment = RegisteredRequests[SelectedRequestIndex].Parameters;
		}

		Updated.Add({ PerViewerRequests[DirtyViewerIndex].Viewer, RequestFragment });
	}

	Context.Defer().PushCommand<FMassDeferredSetCommand>([Updated = MoveTemp(Updated)](const FMassEntityManager& Manager)
		{
			TArray<FMassEntityHandle> EntitiesToSignal;
			EntitiesToSignal.Reserve(Updated.Num());
			for (const TPair<FMassEntityHandle, FMassLookAtRequestFragment>& Pair : Updated)
			{
				const FMassEntityHandle Entity = Pair.Key;
				if (FMassLookAtFragment* LookAtFragment = Manager.IsEntityValid(Entity) ? Manager.GetFragmentDataPtr<FMassLookAtFragment>(Entity) : nullptr)
				{
					const FMassLookAtRequestFragment& Request = Pair.Value;

					// Default request is used to clear the override
					const bool bIsClearOverrideRequest = !Request.ViewerEntity.IsSet();

					switch (LookAtFragment->OverrideState)
					{
					case FMassLookAtFragment::EOverrideState::AllDisabled:
						if (bIsClearOverrideRequest)
						{
							// Already disabled no need to change the fragment
							continue;
						}

						LookAtFragment->OverrideState = FMassLookAtFragment::EOverrideState::ActiveOverrideOnly;
						break;

					case FMassLookAtFragment::EOverrideState::ActiveOverrideOnly:
						if (bIsClearOverrideRequest)
						{
							LookAtFragment->OverrideState = FMassLookAtFragment::EOverrideState::AllDisabled;
						}

						// In both cases we set values from the request to apply the override or to clear the active one
						break;

					case FMassLookAtFragment::EOverrideState::ActiveSystemicOnly:
						if (bIsClearOverrideRequest)
						{
							// Override already cleared, do not change the fragment (systemic is active)
							continue;
						}

						// Switch to 'OverridenSystemic' state and apply request parameters
						LookAtFragment->OverrideState = FMassLookAtFragment::EOverrideState::OverridenSystemic;
						break;

					case FMassLookAtFragment::EOverrideState::OverridenSystemic:
						if (bIsClearOverrideRequest)
						{
							// Mark as pending reactivation and signal the entity to wake up the task to retry activation of the systemic LookAt
							LookAtFragment->OverrideState = FMassLookAtFragment::EOverrideState::PendingSystemicReactivation;
							EntitiesToSignal.Add(Pair.Key);
							continue;
						}
						// Stay in 'OverridenSystemic' state and apply new request parameters
						break;

					case FMassLookAtFragment::EOverrideState::PendingSystemicReactivation:
						if (bIsClearOverrideRequest)
						{
							// Already pending reactivation and signaled, no need to signal again
							continue;
						}

						// Switch back to 'OverridenSystemic' so task will not apply its values when processing the signal
						LookAtFragment->OverrideState = FMassLookAtFragment::EOverrideState::OverridenSystemic;
						break;
					}

					// Only update main LookAt information and don't modify gaze related one
					LookAtFragment->InterpolationSpeed = Request.InterpolationSpeed;
					LookAtFragment->CustomInterpolationSpeed = Request.CustomInterpolationSpeed;
					LookAtFragment->LookAtMode = Request.LookAtMode;
					LookAtFragment->TrackedEntity = Request.TargetEntity;
				}
			}

			// Signal all entities
			if (EntitiesToSignal.Num())
			{
				if (UMassSignalSubsystem* SignalSubsystem = Manager.GetWorld()->GetSubsystem<UMassSignalSubsystem>())
				{
					SignalSubsystem->SignalEntities(UE::Mass::Signals::LookAtFinished, EntitiesToSignal);
				}
			}
		});
}

#if WITH_MASSGAMEPLAY_DEBUG
FString UMassLookAtSubsystem::DebugGetRequestsString(const FMassEntityHandle InEntity) const
{
	UE_MT_SCOPED_READ_ACCESS(RequestsAccessDetector);

	const int32* ViewerDataIndex = ViewerHandleToIndexMap.Find(InEntity);

	TStringBuilder<256> Builder;
	if (ViewerDataIndex && *ViewerDataIndex != INDEX_NONE)
	{
		for (const int32 RequestIndex : PerViewerRequests[*ViewerDataIndex].RequestIndices)
		{
			if (Builder.Len() > 0)
			{
				Builder << TEXT("\n");
			}

			Builder << (RegisteredRequests[RequestIndex].bActive ? TEXT(">") : TEXT("   "));
			Builder << LexToString(RegisteredRequests[RequestIndex].Parameters);
		}
	}
	return Builder.ToString();
}
#endif // WITH_MASSGAMEPLAY_DEBUG
