// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtTargetTrait.h"
#include "GameFramework/Actor.h"
#include "MassLookAtFragments.h"
#include "MassCommonFragments.h"
#include "MassEntityTemplateRegistry.h"
#include "MassEntityView.h"
#include "MassLookAtSettings.h"
#include "Components/CapsuleComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLookAtTargetTrait)

void UMassLookAtTargetTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	const UMassLookAtSettings* Settings = GetDefault<UMassLookAtSettings>();

	BuildContext.AddFragment<FTransformFragment>();
	FMassLookAtTargetFragment& Target = BuildContext.AddFragment_GetRef<FMassLookAtTargetFragment>();
	Target.Offset = Settings->GetDefaultTargetLocationOffset();
	Target.Priority = Priority;

	if (bShouldUseCapsuleComponentToSetTargetOffset)
	{
		BuildContext.GetMutableObjectFragmentInitializers().Add([=](UObject& Owner, const FMassEntityView& EntityView, const EMassTranslationDirection)
			{
				const AActor* AsActor = Cast<AActor>(&Owner);

				const UCapsuleComponent* CapsuleComponent = AsActor
					? AsActor->FindComponentByClass<UCapsuleComponent>()
					: Cast<UCapsuleComponent>(&Owner);

				if (CapsuleComponent != nullptr)
				{
					double Height = 2 * CapsuleComponent->GetScaledCapsuleHalfHeight();
					Height *= Settings->GetTargetHeightRatio();
					Height += Settings->GetFixedOffsetFromTargetHeight();
					EntityView.GetFragmentData<FMassLookAtTargetFragment>().Offset = FVector(0, 0, Height);
				}
			});
	}
}
