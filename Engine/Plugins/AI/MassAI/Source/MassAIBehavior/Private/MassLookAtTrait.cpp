// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtTrait.h"
#include "MassLookAtFragments.h"
#include "MassEntityTemplateRegistry.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLookAtTrait)

void UMassLookAtTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FMassLookAtFragment>();
	BuildContext.AddFragment<FMassLookAtTrajectoryFragment>();
}
