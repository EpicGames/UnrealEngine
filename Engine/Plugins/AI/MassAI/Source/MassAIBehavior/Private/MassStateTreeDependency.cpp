// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStateTreeDependency)

namespace UE::MassBehavior
{

	FStateTreeDependencyBuilder::FStateTreeDependencyBuilder(TArray<FMassStateTreeDependency>& InDependencies)
		: Dependencies(InDependencies)
	{
	}

	void FStateTreeDependencyBuilder::Add(TNotNull<const UStruct*> Struct, FStateTreeDependencyBuilder::EAccessType Access)
	{
		static_assert((int32)EAccessType::ReadOnly == (int32)EMassFragmentAccess::ReadOnly);
		static_assert((int32)EAccessType::ReadWrite == (int32)EMassFragmentAccess::ReadWrite);

		const EMassFragmentAccess FragmentAccess = static_cast<EMassFragmentAccess>(Access);

		if (FMassStateTreeDependency* Found = Dependencies.FindByPredicate([Struct](const FMassStateTreeDependency& Other) { return Other.Type == Struct; }))
		{
			// set the worst case
			if (Found->Access != FragmentAccess && Access == EAccessType::ReadWrite)
			{
				Found->Access = EMassFragmentAccess::ReadWrite;
			}
		}
		else
		{
			Dependencies.Emplace(Struct, FragmentAccess);
		}
	}

} //namespace
