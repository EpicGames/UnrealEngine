// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStateTreeExecutionContext.h"
#include "MassStateTreeTypes.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassSignalSubsystem.h"
#include "MassEntityView.h"
#include "Engine/World.h"
#include "ProfilingDebugging/CsvProfiler.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStateTreeExecutionContext)

CSV_DECLARE_CATEGORY_EXTERN(MassStateTreeProcessor);

namespace UE::MassBehavior
{
bool CollectExternalData(const FStateTreeExecutionContext& Context, const UStateTree* StateTree, TArrayView<const FStateTreeExternalDataDesc> ExternalDataDescs, TArrayView<FStateTreeDataView> OutDataViews)
{
	CSV_SCOPED_TIMING_STAT(MassStateTreeProcessor, CollectExternalData);
	
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<const FMassStateTreeExecutionContext&>(Context); 
	const FMassEntityManager& EntityManager = MassStateTreeContext.GetEntityManager();
	const UWorld* World = MassStateTreeContext.GetWorld();
	
	bool bFoundAll = true;
	const FMassEntityView EntityView(EntityManager, MassStateTreeContext.GetEntity());

	check(ExternalDataDescs.Num() == OutDataViews.Num());

	for (int32 Index = 0; Index < ExternalDataDescs.Num(); Index++)
	{
		const FStateTreeExternalDataDesc& DataDesc = ExternalDataDescs[Index];
		if (DataDesc.Struct == nullptr)
		{
			continue;
		}

		if (UE::Mass::IsA<FMassFragment>(DataDesc.Struct))
		{
			const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(DataDesc.Struct);
			FStructView Fragment = EntityView.GetFragmentDataStruct(ScriptStruct);
			if (Fragment.IsValid())
			{
				OutDataViews[Index] = FStateTreeDataView(Fragment);
			}
			else
			{
				if (DataDesc.Requirement == EStateTreeExternalDataRequirement::Required)
				{
					UE_LOGF(LogMass, Error, "Missing Fragment: %ls", *GetNameSafe(ScriptStruct));

					// Note: Not breaking here, so that we can validate all missing ones in one go.
					bFoundAll = false;
				}
			}
		}
		else if (UE::Mass::IsA<FMassSharedFragment>(DataDesc.Struct))
		{
			const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(DataDesc.Struct);
			FStructView Fragment = EntityView.GetSharedFragmentDataStruct(ScriptStruct);
			if (Fragment.IsValid())
			{
				OutDataViews[Index] = FStateTreeDataView(Fragment.GetScriptStruct(), Fragment.GetMemory());
			}
			else
			{
				if (DataDesc.Requirement == EStateTreeExternalDataRequirement::Required)
				{
					UE_LOGF(LogMass, Error, "Missing Shared Fragment: %ls", *GetNameSafe(ScriptStruct));

					// Note: Not breaking here, so that we can validate all missing ones in one go.
					bFoundAll = false;
				}
			}
		}
		else if (UE::Mass::IsA<FMassConstSharedFragment>(DataDesc.Struct))
		{
			const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(DataDesc.Struct);
			FConstStructView Fragment = EntityView.GetConstSharedFragmentDataStruct(ScriptStruct);
			if (Fragment.IsValid())
			{
				OutDataViews[Index] = FStateTreeDataView(Fragment.GetScriptStruct(), const_cast<uint8*>(Fragment.GetMemory()));
			}
			else
			{
				if (DataDesc.Requirement == EStateTreeExternalDataRequirement::Required)
				{
					UE_LOGF(LogMass, Error, "Missing Const Shared Fragment: %ls", *GetNameSafe(ScriptStruct));

					// Note: Not breaking here, so that we can validate all missing ones in one go.
					bFoundAll = false;
				}
			}
		}
		else if (DataDesc.Struct && DataDesc.Struct->IsChildOf(UWorldSubsystem::StaticClass()))
		{
			const TSubclassOf<UWorldSubsystem> SubClass = Cast<UClass>(const_cast<UStruct*>(ToRawPtr(DataDesc.Struct)));
			UWorldSubsystem* Subsystem = World->GetSubsystemBase(SubClass);
			if (Subsystem)
			{
				OutDataViews[Index] = FStateTreeDataView(Subsystem);
			}
			else
			{
				if (DataDesc.Requirement == EStateTreeExternalDataRequirement::Required)
				{
					UE_LOGF(LogMass, Error, "Missing Subsystem: %ls", *GetNameSafe(SubClass));

					// Note: Not breaking here, so that we can validate all missing ones in one go.
					bFoundAll = false;
				}
			}
		}
	}
	
	return bFoundAll;
}

}; // UE::MassBehavior

FMassStateTreeExecutionContext::FMassStateTreeExecutionContext(UObject& InOwner, const UStateTree& InStateTree, FStateTreeInstanceData& InInstanceData, FMassExecutionContext& InContext)
	: FStateTreeExecutionContext(InOwner, InStateTree, InInstanceData, FOnCollectStateTreeExternalData::CreateStatic(UE::MassBehavior::CollectExternalData))
	, MassEntityExecutionContext(&InContext)
{
}

EStateTreeRunStatus FMassStateTreeExecutionContext::Start()
{
	ensureMsgf(Entity.IsValid(), TEXT("The entity is not valid before starting the state tree instance."));

	FMassExecutionExtension Extension;
	Extension.Entity = Entity;
	return FStateTreeExecutionContext::Start(FStateTreeExecutionContext::FStartParameters
		{
			.ExecutionExtension = TInstancedStruct<FMassExecutionExtension>::Make(MoveTemp(Extension))
		});
}

EStateTreeRunStatus FMassStateTreeExecutionContext::Start(const FInstancedPropertyBag* InitialParameters, int32 RandomSeed)
{
	ensureMsgf(Entity.IsValid(), TEXT("The entity is not valid before starting the state tree instance."));

	const TOptional<int32> ParamRandomSeed = RandomSeed == -1 ? TOptional<int32>() : RandomSeed;

	FMassExecutionExtension Extension;
	Extension.Entity = Entity;
	return FStateTreeExecutionContext::Start(FStateTreeExecutionContext::FStartParameters
		{
			.InitialGlobalParameters = InitialParameters ? InitialParameters->GetValue() : FConstStructView(),
			.ExecutionExtension = TInstancedStruct<FMassExecutionExtension>::Make(MoveTemp(Extension)),
			.RandomSeed = ParamRandomSeed
		});
}

void FMassStateTreeExecutionContext::SetEntity(const FMassEntityHandle InEntity)
{
	Entity = InEntity;

#if WITH_STATETREE_DEBUG
	const TInstancedStruct<FStateTreeExecutionExtension>& ExecutionExtension = Storage.GetExecutionState().ExecutionExtension;
	if (ExecutionExtension.IsValid())
	{
		const FMassExecutionExtension* Extension = ExecutionExtension.GetPtr<const FMassExecutionExtension>();
		ensureMsgf(Extension && Extension->Entity == Entity, TEXT("Incorrect extension type or the entity handle is different from previous execution."));
	}
#endif
}

FString FMassExecutionExtension::GetInstanceDescription(const FContextParameters& Context) const
{
	return FString::Printf(TEXT("Entity [%s]"), *Entity.DebugGetDescription());
}

void FMassExecutionExtension::OnLinkedStateTreeOverridesSet(const FContextParameters& Context, const FStateTreeReferenceOverrides& Overrides)
{
	const uint32 NewLinkedStateTreeOverridesHash = GetTypeHash(Overrides);
	if (NewLinkedStateTreeOverridesHash != LinkedStateTreeOverridesHash)
	{
		LinkedStateTreeOverridesHash = NewLinkedStateTreeOverridesHash;
		//@todo update mass dependencies
	}
}

void FMassStateTreeExecutionContext::BeginDelayedTransition(const FStateTreeTransitionDelayedState& DelayedState)
{
	UMassSignalSubsystem* SignalSubsystem = MassEntityExecutionContext->GetMutableSubsystem<UMassSignalSubsystem>();
	if (SignalSubsystem != nullptr && Entity.IsSet())
	{
		// Tick again after the games time has passed to see if the condition still holds true.
		SignalSubsystem->DelaySignalEntityDeferred(GetMassEntityExecutionContext(), UE::Mass::Signals::DelayedTransitionWakeup, Entity, DelayedState.TimeLeft + KINDA_SMALL_NUMBER);
	}
}
