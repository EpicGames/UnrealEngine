// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStateTreeProcessors.h"
#include "Containers/ConsumeAllMpmcQueue.h"
#include "Engine/World.h"
#include "Experimental/ConcurrentLinearAllocator.h"
#include "MassBehaviorSettings.h"
#include "MassComponentHitTypes.h"
#include "MassDebugger.h"
#include "MassExecutionContext.h"
#include "MassNavigationTypes.h"
#include "MassSignalSubsystem.h"
#include "MassSimulationLOD.h"
#include "MassSmartObjectTypes.h"
#include "MassStateTreeExecutionContext.h"
#include "MassStateTreeFragments.h"
#include "MassStateTreeSubsystem.h"
#include "MassZoneGraphAnnotationTypes.h"
#include "ObjectTrace.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "StateTree.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStateTreeProcessors)

CSV_DEFINE_CATEGORY(MassStateTreeProcessor, false);

namespace UE::MassBehavior
{

template<typename TFunc>
void ForEachEntityInChunk(
	FMassExecutionContext& Context,
	UMassStateTreeSubsystem& MassStateTreeSubsystem,
	TFunc&& Callback)
{
	const TArrayView<FMassStateTreeInstanceFragment> StateTreeInstanceList = Context.GetMutableFragmentView<FMassStateTreeInstanceFragment>();
	const FMassStateTreeSharedFragment& SharedStateTree = Context.GetConstSharedFragment<FMassStateTreeSharedFragment>();

	// Assuming that all the entities share same StateTree, because they all should have the same storage fragment.
	const int32 NumEntities = Context.GetNumEntities();
	check(NumEntities > 0);
	
	const UStateTree* StateTree = SharedStateTree.StateTree;

	for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
	{
		const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
		FMassStateTreeInstanceFragment& StateTreeFragment = StateTreeInstanceList[EntityIt];
		FStateTreeInstanceData* InstanceData = MassStateTreeSubsystem.GetInstanceData(StateTreeFragment.InstanceHandle);
		if (InstanceData != nullptr)
		{
			const UObject* TraceOwner = Context.GetEntityManagerChecked().GetOwner();
			TRACE_OBJECT(TraceOwner);
			TRACE_INSTANCE(TraceOwner, Entity.AsNumber(), FObjectTrace::GetObjectId(TraceOwner), FMassEntityHandle::StaticStruct(), FString::Printf(TEXT("Entity[%d:%d]"), Entity.Index, Entity.SerialNumber));

			FMassStateTreeExecutionContext StateTreeContext(MassStateTreeSubsystem, *StateTree, *InstanceData, Context);
			StateTreeContext.SetEntity(Entity);
			StateTreeContext.SetOuterTraceId(Entity.AsNumber());

			// Make sure all required external data are set.
			{
				CSV_SCOPED_TIMING_STAT(MassStateTreeProcessor, ExternalDataValidation);
				// TODO: disable this when not in debug.
				if (!ensureMsgf(StateTreeContext.AreContextDataViewsValid(), TEXT("StateTree will not execute due to missing external data.")))
				{
					break;
				}
			}

			Callback(StateTreeContext, StateTreeFragment);
		}
	}
}
} // UE::MassBehavior

namespace UE::MassStateTree
{
struct FBlockAllocationTag : FDefaultBlockAllocationTag
{
	static constexpr const char* TagName = "MassStateTreeLinear";
	using Allocator = FAlignedAllocator;
};

struct FLinearQueueAllocator
{
	FORCEINLINE static void* Malloc(const SIZE_T Size, const uint32 Alignment)
	{
		return TConcurrentLinearAllocator<FBlockAllocationTag>::Malloc(Size, Alignment);
	}

	FORCEINLINE static void Free(void* Pointer)
	{
		TConcurrentLinearAllocator<FBlockAllocationTag>::Free(Pointer);
	}
};

} // UE::MassStateTree

//----------------------------------------------------------------------//
// UMassStateTreeFragmentDestructor
//----------------------------------------------------------------------//
UMassStateTreeFragmentDestructor::UMassStateTreeFragmentDestructor()
	: EntityQuery(*this)
{
	ExecutionFlags = static_cast<uint8>(UE::MassStateTree::ExecutionFlags);
	ObservedTypes.Add(FMassStateTreeInstanceFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	bRequiresGameThreadExecution = true;
}

void UMassStateTreeFragmentDestructor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(Owner.GetWorld());
	Super::InitializeInternal(Owner, EntityManager);
}

void UMassStateTreeFragmentDestructor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassStateTreeInstanceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassStateTreeSharedFragment>();
	EntityQuery.AddSubsystemRequirement<UMassStateTreeSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassStateTreeFragmentDestructor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (SignalSubsystem == nullptr)
	{
		return;
	}

	EntityQuery.ForEachEntityChunk(Context,
		[](FMassExecutionContext& Context)
		{
			UMassStateTreeSubsystem& MassStateTreeSubsystem = Context.GetMutableSubsystemChecked<UMassStateTreeSubsystem>();
			const TArrayView<FMassStateTreeInstanceFragment> StateTreeInstanceList = Context.GetMutableFragmentView<FMassStateTreeInstanceFragment>();

			UE::MassBehavior::ForEachEntityInChunk(Context, MassStateTreeSubsystem,
				[](FMassStateTreeExecutionContext& StateTreeExecutionContext, FMassStateTreeInstanceFragment&)
				{
					// Stop the tree instance
					StateTreeExecutionContext.Stop();
				});

			// Free the StateTree instance memory
			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassStateTreeInstanceFragment& StateTreeInstance = StateTreeInstanceList[EntityIt];
				if (StateTreeInstance.InstanceHandle.IsValid())
				{
					MassStateTreeSubsystem.FreeInstanceData(StateTreeInstance.InstanceHandle);
					StateTreeInstance.InstanceHandle = FMassStateTreeInstanceHandle();
				}
			}
		});
}

//----------------------------------------------------------------------//
// UMassStateTreeActivationProcessor
//----------------------------------------------------------------------//
UMassStateTreeActivationProcessor::UMassStateTreeActivationProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = static_cast<uint8>(UE::MassStateTree::ExecutionFlags);
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Behavior);
	bRequiresGameThreadExecution = true; // due to UMassStateTreeSubsystem RW access
}

void UMassStateTreeActivationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassStateTreeInstanceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassStateTreeSharedFragment>();
	EntityQuery.AddTagRequirement<FMassStateTreeActivatedTag>(EMassFragmentPresence::None);
	EntityQuery.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.AddSubsystemRequirement<UMassStateTreeSubsystem>(EMassFragmentAccess::ReadWrite);

	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassStateTreeActivationProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& ExecutionContext)
{
	UMassSignalSubsystem& SignalSubsystem = ExecutionContext.GetMutableSubsystemChecked<UMassSignalSubsystem>();

	const UMassBehaviorSettings* BehaviorSettings = GetDefault<UMassBehaviorSettings>();
	check(BehaviorSettings);
 
	// StateTree processor relies on signals to be ticked, but we need an 'initial tick' to set the tree in the proper state.
	// The initializer provides that by sending a signal to all new entities that use StateTree.
	const double TimeInSeconds = EntityManager.GetWorld()->GetTimeSeconds();

	TArray<FMassEntityHandle> EntitiesToSignal;
	int32 ActivationCounts[EMassLOD::Max] {0,0,0,0};
	
	EntityQuery.ForEachEntityChunk(ExecutionContext,
		[&EntitiesToSignal, &ActivationCounts, MaxActivationsPerLOD = BehaviorSettings->MaxActivationsPerLOD, TimeInSeconds](FMassExecutionContext& Context)
		{
			UMassStateTreeSubsystem& MassStateTreeSubsystem = Context.GetMutableSubsystemChecked<UMassStateTreeSubsystem>();
			const int32 NumEntities = Context.GetNumEntities();
			const EMassLOD::Type ChunkLOD = FMassSimulationVariableTickChunkFragment::GetChunkLOD(Context);
			checkf(ChunkLOD != EMassLOD::Type::Max, TEXT("Attempting to activate a state tree with an uninitialized LOD"));

			// Check if we already reached the maximum for this frame
			if (ActivationCounts[ChunkLOD] > MaxActivationsPerLOD[ChunkLOD])
			{
				return;
			}
			ActivationCounts[ChunkLOD] += NumEntities;

			const TArrayView<FMassStateTreeInstanceFragment> StateTreeInstanceList = Context.GetMutableFragmentView<FMassStateTreeInstanceFragment>();
			const FMassStateTreeSharedFragment& SharedStateTree = Context.GetConstSharedFragment<FMassStateTreeSharedFragment>();

			// Allocate and initialize the StateTree instance memory
			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FMassStateTreeInstanceFragment& StateTreeInstance = StateTreeInstanceList[EntityIt];
				StateTreeInstance.InstanceHandle = MassStateTreeSubsystem.AllocateInstanceData(SharedStateTree.StateTree);
			}
			
			// Start StateTree. This may do substantial amount of work, as we select and enter the first state.
			UE::MassBehavior::ForEachEntityInChunk(Context, MassStateTreeSubsystem,
				[TimeInSeconds](FMassStateTreeExecutionContext& StateTreeExecutionContext, FMassStateTreeInstanceFragment& StateTreeFragment)
				{
					// Start the tree instance
					StateTreeExecutionContext.Start();
					StateTreeFragment.LastUpdateTimeInSeconds = TimeInSeconds;
				});

			// Adding a tag on each entity to remember we have sent the state tree initialization signal
			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassStateTreeInstanceFragment& StateTreeInstance = StateTreeInstanceList[EntityIt];
				if (StateTreeInstance.InstanceHandle.IsValid())
				{
					const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
					Context.Defer().AddTag<FMassStateTreeActivatedTag>(Entity);
					EntitiesToSignal.Add(Entity);
				}
			}
		});
	
	// Signal all entities inside the consolidated list
	if (EntitiesToSignal.Num())
	{
		SignalSubsystem.SignalEntities(UE::Mass::Signals::StateTreeActivate, EntitiesToSignal);
	}
}

//----------------------------------------------------------------------//
// UMassStateTreeProcessor
//----------------------------------------------------------------------//
UMassStateTreeProcessor::UMassStateTreeProcessor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bRequiresGameThreadExecution = bProcessEntitiesInParallel;

	ExecutionFlags = static_cast<uint8>(UE::MassStateTree::ExecutionFlags);
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Behavior;

	// `Behavior` doesn't run on clients but `Tasks` do.
	// We define the dependencies here so task won't need to set their dependency on `Behavior`,
	// but only on `SyncWorldToMass`
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SyncWorldToMass);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Tasks);

	if (UE::Mass::StateTree::bDynamicSTProcessorsEnabled)
	{
		bAutoRegisterWithProcessingPhases = false;
		bAllowMultipleInstances = true;
	}
}

void UMassStateTreeProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(Owner.GetWorld());

	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::StateTreeActivate);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::LookAtFinished);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::NewStateTreeTaskRequired);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::StandTaskFinished);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::DelayedTransitionWakeup);

	// @todo MassStateTree: add a way to register/unregister from enter/exit state (need reference counting)
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SmartObjectRequestCandidates);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SmartObjectCandidatesReady);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SmartObjectInteractionDone);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::SmartObjectInteractionAborted);

	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::FollowPointPathStart);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::FollowPointPathDone);
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::CurrentLaneChanged);

	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::AnnotationTagsChanged);

	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::HitReceived);

	// @todo MassStateTree: move this to its game plugin when possible
	SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::ContextualAnimTaskFinished);
}

void UMassStateTreeProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassStateTreeInstanceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddConstSharedRequirement<FMassStateTreeSharedFragment>();
	EntityQuery.AddSubsystemRequirement<UMassStateTreeSubsystem>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);

	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassStateTreeProcessor::SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& EntitySignals)
{
	UMassSignalSubsystem& SignalSubsystem = Context.GetMutableSubsystemChecked<UMassSignalSubsystem>();
	
	QUICK_SCOPE_CYCLE_COUNTER(StateTreeProcessor_Run);
	CSV_SCOPED_TIMING_STAT(MassStateTreeProcessor, Execute);

	const double TimeInSeconds = EntityManager.GetWorld()->GetTimeSeconds();

	auto TickFunc = [TimeInSeconds, &EntitySignals](FMassExecutionContext& ExecutionContext, TFunctionRef<void(FMassEntityHandle)> AddEntityToSignalFunc)
		{
			// Keep stats regarding the amount of tree instances ticked per frame
			CSV_CUSTOM_STAT(MassStateTreeProcessor, NumTickedStateTree, ExecutionContext.GetNumEntities(), ECsvCustomStatOp::Accumulate);

			UMassStateTreeSubsystem& MassStateTreeSubsystem = ExecutionContext.GetMutableSubsystemChecked<UMassStateTreeSubsystem>();

			UE::MassBehavior::ForEachEntityInChunk(ExecutionContext, MassStateTreeSubsystem,
				[TimeInSeconds, AddEntityToSignalFunc, &EntitySignals, &MassStateTreeSubsystem]
				(FMassStateTreeExecutionContext& StateTreeExecutionContext, FMassStateTreeInstanceFragment& StateTreeFragment)
				{
					// Compute adjusted delta time
					const float AdjustedDeltaTime = FloatCastChecked<float>(TimeInSeconds - StateTreeFragment.LastUpdateTimeInSeconds, /* Precision */ 1./256.);
					StateTreeFragment.LastUpdateTimeInSeconds = TimeInSeconds;

#if WITH_MASSGAMEPLAY_DEBUG
					const FMassEntityHandle Entity = StateTreeExecutionContext.GetEntity();
					if (UE::Mass::Debug::IsDebuggingEntity(Entity))
					{
						TArray<FName> Signals;
						EntitySignals.GetSignalsForEntity(Entity, Signals);
						FString SignalsString;
						for (const FName& SignalName : Signals)
						{
							if (!SignalsString.IsEmpty())
							{
								SignalsString += TEXT(", ");
							}
							SignalsString += SignalName.ToString();
						}
						UE_VLOG_UELOG(&MassStateTreeSubsystem, LogStateTree, Log, TEXT("%s: Ticking StateTree because of signals: %s"), *Entity.DebugGetDescription(), *SignalsString);
					}
#endif // WITH_MASSGAMEPLAY_DEBUG

					// Tick the tree instance
					StateTreeExecutionContext.Tick(AdjustedDeltaTime);

					// When last tick status is different from "Running", the state tree need to be tick again
					// For performance reason, tick again to see if we could find a new state right away instead of waiting the next frame.
					if (StateTreeExecutionContext.GetLastTickStatus() != EStateTreeRunStatus::Running)
					{
						StateTreeExecutionContext.Tick(0.0f);

						// Could not find new state yet, try again next frame
						if (StateTreeExecutionContext.GetLastTickStatus() != EStateTreeRunStatus::Running)
						{
							AddEntityToSignalFunc(StateTreeExecutionContext.GetEntity());
						}
					}
				});
		};

	TArray<FMassEntityHandle, FNonconcurrentLinearArrayAllocator> EntitiesToSignal;
	if (bProcessEntitiesInParallel)
	{
		UE::TConsumeAllMpmcQueue<FMassEntityHandle, UE::MassStateTree::FLinearQueueAllocator> EntitiesToSignalQueue;

		EntityQuery.ParallelForEachEntityChunk(Context, [TickFunc, &EntitiesToSignalQueue](FMassExecutionContext& ExecutionContext)
			{
				TickFunc(ExecutionContext, [&EntitiesToSignalQueue](FMassEntityHandle Entity)
					{
						EntitiesToSignalQueue.ProduceItem(Entity);
					});
			});

		EntitiesToSignalQueue.ConsumeAllFifo([&EntitiesToSignal](FMassEntityHandle&& Entity)
			{
				EntitiesToSignal.Add(MoveTemp(Entity));
			});
	}
	else
	{
		EntityQuery.ForEachEntityChunk(Context, [TickFunc, &EntitiesToSignal](FMassExecutionContext& ExecutionContext)
			{
				TickFunc(ExecutionContext, [&EntitiesToSignal](FMassEntityHandle Entity)
					{
						EntitiesToSignal.Add(Entity);
					});
			});
	}

	if (EntitiesToSignal.Num() > 0)
	{
		SignalSubsystem.SignalEntities(UE::Mass::Signals::NewStateTreeTaskRequired, EntitiesToSignal);
	}
}

void UMassStateTreeProcessor::SetExecutionRequirements(const FMassFragmentRequirements& FragmentRequirements
	, const FMassSubsystemRequirements& SubsystemRequirements)
{
	if (!ensureMsgf(IsInitialized() == false, TEXT("%hs: calling after processor's initialization is not supported."), __FUNCTION__))
	{
		return;
	}
	FragmentRequirements.ExportRequirements(ExecutionRequirements);
	SubsystemRequirements.ExportRequirements(ExecutionRequirements);
}

void UMassStateTreeProcessor::ExportRequirements(FMassExecutionRequirements& OutRequirements) const
{
	Super::ExportRequirements(OutRequirements);
	OutRequirements.Append(ExecutionRequirements);
}

void UMassStateTreeProcessor::AddHandledStateTree(TNotNull<const UStateTree*> StateTree)
{
	HandledStateTrees.AddUnique(StateTree);

	// need to clear the filter first. This is a safety-feature to ensure filters do not get accidentally overridden 
	EntityQuery.ClearChunkFilter();
	EntityQuery.SetChunkFilter([HandledStateTreesCopy = HandledStateTrees](const FMassExecutionContext& Context) -> bool
	{
		const FMassStateTreeSharedFragment& SharedStateTree = Context.GetConstSharedFragment<FMassStateTreeSharedFragment>();
		return HandledStateTreesCopy.Find(SharedStateTree.StateTree) != INDEX_NONE;
	});
}
