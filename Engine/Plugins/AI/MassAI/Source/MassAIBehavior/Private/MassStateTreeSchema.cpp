// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStateTreeSchema.h"
#include "MassEntityTypes.h"
#include "MassStateTreeDependency.h"
#include "MassStateTreeTypes.h"
#include "StateTree.h"
#include "StateTreeConditionBase.h"
#include "StateTreeConsiderationBase.h"
#include "StateTreeExecutionTypes.h"
#include "StateTreeLinker.h"
#include "Subsystems/WorldSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStateTreeSchema)

#ifndef UE_WITH_MASS_STATETREE_DEPENDENCIES_DEBUG
#define UE_WITH_MASS_STATETREE_DEPENDENCIES_DEBUG 0
#endif

bool UMassStateTreeSchema::IsStructAllowed(const UScriptStruct* InScriptStruct) const
{
	// Only allow Mass and common structs.
	return InScriptStruct->IsChildOf(FMassStateTreeEvaluatorBase::StaticStruct())
			|| InScriptStruct->IsChildOf(FMassStateTreeTaskBase::StaticStruct())
			|| InScriptStruct->IsChildOf(FMassStateTreeConditionBase::StaticStruct())
			|| InScriptStruct->IsChildOf(FMassStateTreePropertyFunctionBase::StaticStruct())
			|| InScriptStruct->IsChildOf(FStateTreeEvaluatorCommonBase::StaticStruct())
			|| InScriptStruct->IsChildOf(FStateTreeConditionCommonBase::StaticStruct())
			|| InScriptStruct->IsChildOf(FStateTreeConsiderationCommonBase::StaticStruct())
			|| InScriptStruct->IsChildOf(FStateTreePropertyFunctionCommonBase::StaticStruct());
}

bool UMassStateTreeSchema::IsExternalItemAllowed(const UStruct& InStruct) const
{
	// Allow only WorldSubsystems and fragments as external data.
	return InStruct.IsChildOf(UWorldSubsystem::StaticClass())
			|| UE::Mass::IsA<FMassFragment>(&InStruct)
			|| UE::Mass::IsA<FMassSharedFragment>(&InStruct)
			|| UE::Mass::IsA<FMassConstSharedFragment>(&InStruct);
}

bool UMassStateTreeSchema::Link(FStateTreeLinker& Linker)
{
	Dependencies.Empty();
	UE::MassBehavior::FStateTreeDependencyBuilder Builder(Dependencies);

	auto BuildDependencies = [&Builder](const UStateTree* StateTree)
		{
			for (FConstStructView Node : StateTree->GetNodes())
			{
				if (const FMassStateTreeEvaluatorBase* Evaluator = Node.GetPtr<const FMassStateTreeEvaluatorBase>())
				{
					Evaluator->GetDependencies(Builder);
				}
				else if (const FMassStateTreeTaskBase* Task = Node.GetPtr<const FMassStateTreeTaskBase>())
				{
					Task->GetDependencies(Builder);
				}
				else if (const FMassStateTreeConditionBase* Condition = Node.GetPtr<const FMassStateTreeConditionBase>())
				{
					Condition->GetDependencies(Builder);
				}
				else if (const FMassStateTreePropertyFunctionBase* PropertyFunction = Node.GetPtr<const FMassStateTreePropertyFunctionBase>())
				{
					PropertyFunction->GetDependencies(Builder);
				}
			}
		};

	TArray<const UStateTree*, TInlineAllocator<4>> StateTrees;
	StateTrees.Add(CastChecked<const UStateTree>(GetOuter()));
	for (int32 Index = 0; Index < StateTrees.Num(); ++Index)
	{
		const UStateTree* StateTree = StateTrees[Index];
		BuildDependencies(StateTree);

		// The StateTree::Link order is not deterministic. Build the dependencies and add them to this list.
		for (const FCompactStateTreeState& State : StateTree->GetStates())
		{
			if (State.LinkedAsset && State.Type == EStateTreeStateType::LinkedAsset)
			{
				StateTrees.AddUnique(State.LinkedAsset);
			}
		}
	}

#if UE_WITH_MASS_STATETREE_DEPENDENCIES_DEBUG
	for (const FStateTreeExternalDataDesc& Desc : Linker.GetExternalDataDescs())
	{
		const bool bContains = Dependencies.ContainsByPredicate([&Desc](const FMassStateTreeDependency& Other) { return Desc.Struct == Other.Type; });
		ensureMsgf(bContains, TEXT("Tree %s is missing a mass dependency"), *StateTree->GetPathName());
	}
#endif

	return true;
}
