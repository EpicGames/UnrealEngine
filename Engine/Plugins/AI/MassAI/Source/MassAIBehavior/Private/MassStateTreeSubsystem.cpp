// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStateTreeSubsystem.h"
#include "MassAIBehaviorTypes.h"
#include "MassEntityQuery.h"
#include "MassEntitySubsystem.h"
#include "MassProcessorDependencySolver.h"
#include "MassSimulationSubsystem.h"
#include "MassStateTreeSchema.h"
#include "StateTree.h"
#include "Engine/Engine.h"
#include "MassStateTreeProcessors.h"
#include "MassBehaviorSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStateTreeSubsystem)


namespace UE::Mass::StateTree
{
	bool bDynamicSTProcessorsEnabled = true;
	FAutoConsoleVariableRef CVarDynamicSTEnabled(TEXT("ai.mass.DynamicSTProcessorsEnabled"), bDynamicSTProcessorsEnabled
		, TEXT("Whether Dynamic ST processors will be created per distinct ST requirements. Can only be set via code or ini.")
		, ECVF_ReadOnly);
}

void UMassStateTreeSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);

	UMassEntitySubsystem* EntitySubsystem = Collection.InitializeDependency<UMassEntitySubsystem>();
	check(EntitySubsystem);
	ĘntityManager = EntitySubsystem->GetMutableEntityManager().AsShared();

	const UMassBehaviorSettings* BehaviorSettings = GetDefault<UMassBehaviorSettings>();
	check(BehaviorSettings);

	UClass* LoadedClass = BehaviorSettings->DynamicStateTreeProcessorClass.LoadSynchronous();
	DynamicProcessorClass = LoadedClass
		? LoadedClass
		: UMassStateTreeProcessor::StaticClass();

	SimulationSubsystem = Collection.InitializeDependency<UMassSimulationSubsystem>();
}

FMassStateTreeInstanceHandle UMassStateTreeSubsystem::AllocateInstanceData(const UStateTree* StateTree)
{
	if (StateTree == nullptr)
	{
		return FMassStateTreeInstanceHandle();
	}

	UE_MT_SCOPED_WRITE_ACCESS(InstanceDataMTDetector);
	int32 Index = 0;
	if (InstanceDataFreelist.Num() > 0)
	{
		Index = InstanceDataFreelist.Pop();
	}
	else
	{
		Index = InstanceDataArray.Num();
		InstanceDataArray.AddDefaulted();
	}

	FMassStateTreeInstanceDataItem& Item = InstanceDataArray[Index];
	Item.InstanceData.Reset();

	if (UE::Mass::StateTree::bDynamicSTProcessorsEnabled
		&& StateTreeToProcessor.Find(StateTree) == nullptr)
	{
		CreateProcessorForStateTree(StateTree);
	}
	
	return FMassStateTreeInstanceHandle::Make(Index, Item.Generation);
}

void UMassStateTreeSubsystem::FreeInstanceData(const FMassStateTreeInstanceHandle Handle)
{
	if (!IsValidHandle(Handle))
	{
		return;
	}

	UE_MT_SCOPED_WRITE_ACCESS(InstanceDataMTDetector);
	FMassStateTreeInstanceDataItem& Item = InstanceDataArray[Handle.GetIndex()];
	Item.InstanceData.Reset();
	Item.Generation++;

	InstanceDataFreelist.Add(Handle.GetIndex());
}

void UMassStateTreeSubsystem::CreateProcessorForStateTree(TNotNull<const UStateTree*> StateTree)
{
	const UMassStateTreeSchema* StateTreeSchema = Cast<UMassStateTreeSchema>(StateTree->GetSchema());

	FMassSubsystemRequirements SubsystemRequirements;
	FMassFragmentRequirements FragmentRequirements(ĘntityManager);

	if (ensure(StateTreeSchema))
	{
		const TConstArrayView<FMassStateTreeDependency> DependenciesView = StateTreeSchema->GetDependencies();
		if (DependenciesView.Num())
		{
			// Convert loosely defined dependencies (expressed with UStruct pointers)
			// to strongly-typed Mass requirements.
			// The functions below will complain if anything is misconfigured.
			for (const FMassStateTreeDependency& Dependency : DependenciesView)
			{
				if (Dependency.Type)
				{
					if (const UScriptStruct* ScriptStruct = Cast<const UScriptStruct>(Dependency.Type.Get()))
					{
						if (UE::Mass::IsA<FMassFragment>(Dependency.Type))
						{
							FragmentRequirements.AddRequirement(ScriptStruct, Dependency.Access);
						}
						else if (UE::Mass::IsA<FMassTag>(Dependency.Type))
						{
							FragmentRequirements.AddTagRequirement(*ScriptStruct, EMassFragmentPresence::All);
						}
						else if (UE::Mass::IsA<FMassChunkFragment>(Dependency.Type))
						{
							FragmentRequirements.AddChunkRequirement(ScriptStruct, Dependency.Access);
						}
						else if (UE::Mass::IsA<FMassSharedFragment>(Dependency.Type))
						{
							FragmentRequirements.AddSharedRequirement(ScriptStruct, Dependency.Access);
						}
						else if (UE::Mass::IsA<FMassConstSharedFragment>(Dependency.Type))
						{
							FragmentRequirements.AddConstSharedRequirement(ScriptStruct);
						}
						else
						{
							UE_LOGF(LogMassBehavior, Error, "Unhandled Mass State Tree dependency %ls", *Dependency.Type->GetName());
						}
					}
					else if (Dependency.Type->IsChildOf(USubsystem::StaticClass()))
					{
						check(ĘntityManager);
						TSubclassOf<USubsystem> SubsystemClass = Cast<UClass>(const_cast<UStruct*>(Dependency.Type.Get()));
						SubsystemRequirements.AddSubsystemRequirement(SubsystemClass, Dependency.Access, ĘntityManager.ToSharedRef());
					}
					else
					{
						UE_LOGF(LogMassBehavior, Error, "Unhandled Mass State Tree dependency user-type %ls", *Dependency.Type->GetName());
					}
				}
			}
		}
	}

	UMassStateTreeProcessor* DynamicProcessor = nullptr;

	const uint32 DependenciesHash = HashCombine(GetTypeHash(FragmentRequirements), GetTypeHash(SubsystemRequirements));
	TObjectPtr<UMassStateTreeProcessor>& ExistingProcessor = RequirementsHashToProcessor.FindOrAddByHash(DependenciesHash, DependenciesHash);

	if (ExistingProcessor == nullptr)
	{
		DynamicProcessor = NewObject<UMassStateTreeProcessor>(this, DynamicProcessorClass);
		checkf(DynamicProcessor, TEXT("Failed to spawn an instance of %s"), *GetNameSafe(DynamicProcessorClass));
		DynamicProcessor->SetExecutionRequirements(FragmentRequirements, SubsystemRequirements);
		DynamicProcessor->CallInitialize(this, ĘntityManager.ToSharedRef());

		SimulationSubsystem->RegisterDynamicProcessor(*DynamicProcessor);

		ExistingProcessor = DynamicProcessor;
	}
	else
	{
		DynamicProcessor = ExistingProcessor;
	}

	DynamicProcessor->AddHandledStateTree(StateTree);
	
	StateTreeToProcessor.Add(StateTree, DynamicProcessor);
}
