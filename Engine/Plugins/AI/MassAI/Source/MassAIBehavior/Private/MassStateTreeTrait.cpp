// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassStateTreeTrait.h"
#include "MassStateTreeFragments.h"
#include "MassStateTreeSubsystem.h"
#include "MassAIBehaviorTypes.h"
#include "StateTree.h"
#include "VisualLogger/VisualLogger.h"
#include "MassEntityTemplateRegistry.h"
#include "Engine/World.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassStateTreeTrait)


void UMassStateTreeTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	const EProcessorExecutionFlags WorldExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(&World);
	if (!EnumHasAnyFlags(WorldExecutionFlags, UE::MassStateTree::ExecutionFlags))
	{
		UE_VLOG_UELOG(&World, LogMassBehavior, Verbose, TEXT("StateTree doesn't run on the current execution mode '%s'"), *UEnum::GetValueAsString(WorldExecutionFlags));
		return;
	}

	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	UMassStateTreeSubsystem* MassStateTreeSubsystem = World.GetSubsystem<UMassStateTreeSubsystem>();
	if (!MassStateTreeSubsystem && !BuildContext.IsInspectingData())
	{
		UE_VLOG_UELOG(&World, LogMassBehavior, Error, TEXT("Failed to get Mass StateTree Subsystem."));
		return;
	}

	if (!StateTree && !BuildContext.IsInspectingData())
	{
		UE_VLOG_UELOG(MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree asset is not set or unavailable."));
		return;
	}
	if (StateTree != nullptr && !BuildContext.IsInspectingData() && !StateTree->IsReadyToRun())
	{
		UE_VLOG_UELOG(MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree asset is ready to run."));
		return;
	}

	FMassStateTreeSharedFragment SharedStateTree;
	SharedStateTree.StateTree = StateTree;
	
	const FConstSharedStruct StateTreeFragment = EntityManager.GetOrCreateConstSharedFragment(SharedStateTree);
	BuildContext.AddConstSharedFragment(StateTreeFragment);

	BuildContext.AddFragment<FMassStateTreeInstanceFragment>();
}

bool UMassStateTreeTrait::ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const
{
	constexpr EProcessorExecutionFlags ExecutionFlags(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);
	const EProcessorExecutionFlags WorldExecutionFlags = UE::Mass::Utils::DetermineProcessorExecutionFlags(&World);
	if (!EnumHasAnyFlags(WorldExecutionFlags, ExecutionFlags))
	{
		UE_VLOG_UELOG(&World, LogMassBehavior, Verbose, TEXT("StateTree doesn't run on the current execution mode '%s'"), *UEnum::GetValueAsString(WorldExecutionFlags));
		return true;
	}

	UMassStateTreeSubsystem* MassStateTreeSubsystem = World.GetSubsystem<UMassStateTreeSubsystem>();
	if (!MassStateTreeSubsystem)
	{
		UE_VLOG_UELOG(&World, LogMassBehavior, Error, TEXT("Failed to get Mass StateTree Subsystem."));
		return false;
	}

	if (!StateTree)
	{
		UE_VLOG_UELOG(MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree asset is not set or unavailable."));
		return false;
	}

	// Make sure all the required subsystems can be found.
	bool bIssuesFound = false;
	for (const FStateTreeExternalDataDesc& ItemDesc : StateTree->GetExternalDataDescs())
	{
		if (ensure(ItemDesc.Struct) && ItemDesc.Requirement == EStateTreeExternalDataRequirement::Required)
		{
			if (ItemDesc.Struct->IsChildOf(UWorldSubsystem::StaticClass()))
			{
				if (BuildContext.IsInspectingData() == false)
				{
					const TSubclassOf<UWorldSubsystem> SubClass = Cast<UClass>(const_cast<UStruct*>(ToRawPtr(ItemDesc.Struct)));
					USubsystem* Subsystem = World.GetSubsystemBase(SubClass);
					UE_CVLOG_UELOG(!Subsystem, MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree %s: Could not find required subsystem %s"), *GetNameSafe(StateTree), *GetNameSafe(ItemDesc.Struct));
					bIssuesFound = bIssuesFound || !Subsystem;
				}
			}
			else if (UE::Mass::IsA<FMassFragment>(ItemDesc.Struct))
			{
				const UScriptStruct* FragmentType = CastChecked<UScriptStruct>(ItemDesc.Struct);
				if (BuildContext.HasFragment(FragmentType) == false)
				{
					OutTraitRequirements.Add(FragmentType);
					UE_VLOG_UELOG(MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree %s: Could not find required fragment %s"), *GetNameSafe(StateTree), *GetNameSafe(ItemDesc.Struct));
					bIssuesFound = true;
				}
			}
			else if (UE::Mass::IsA<FMassSharedFragment>(ItemDesc.Struct))
			{
				const UScriptStruct* FragmentType = CastChecked<UScriptStruct>(ItemDesc.Struct);
				if (BuildContext.HasSharedFragment(FragmentType) == false)
				{
					OutTraitRequirements.Add(FragmentType);
					UE_VLOG_UELOG(MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree %s: Could not find required shared fragment %s"), *GetNameSafe(StateTree), *GetNameSafe(ItemDesc.Struct));
					bIssuesFound = true;
				}
			}
			else if (UE::Mass::IsA<FMassConstSharedFragment>(ItemDesc.Struct))
			{
				const UScriptStruct* FragmentType = CastChecked<UScriptStruct>(ItemDesc.Struct);
				if (BuildContext.HasConstSharedFragment(FragmentType) == false)
				{
					OutTraitRequirements.Add(FragmentType);
					UE_VLOG_UELOG(MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree %s: Could not find required const shared fragment %s"), *GetNameSafe(StateTree), *GetNameSafe(ItemDesc.Struct));
					bIssuesFound = true;
				}
			}
			else
			{
				UE_VLOG_UELOG(MassStateTreeSubsystem, LogMassBehavior, Error, TEXT("StateTree %s: Unsupported requirement %s"), *GetNameSafe(StateTree), *GetNameSafe(ItemDesc.Struct));
				bIssuesFound = true;
			}
		}
	}

	return !bIssuesFound;
}
