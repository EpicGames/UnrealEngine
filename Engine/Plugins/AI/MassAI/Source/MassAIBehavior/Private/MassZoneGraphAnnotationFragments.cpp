// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassZoneGraphAnnotationFragments.h"
#include "MassEntityManager.h"
#include "MassExecutionContext.h"
#include "MassSimulationLOD.h"
#include "MassDebugger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphAnnotationFragments)


namespace UE::Mass::ZoneGraphAnnotations
{
// Update interval range for periodic annotation tag update.
static const float MinUpdateInterval = 0.25f;
static const float MaxUpdateInterval = 0.5f;

// Update interval range for periodic annotation tag update for Off LOD.
static const float OffLODMinUpdateInterval = 1.905f;
static const float OffLODMaxUpdateInterval = 2.10f;

} // UE::Mass::ZoneGraphAnnotations

//----------------------------------------------------------------------//
//  FMassZoneGraphAnnotationVariableTickChunkFragment
//----------------------------------------------------------------------//

bool FMassZoneGraphAnnotationVariableTickChunkFragment::UpdateChunk(FMassExecutionContext& Context)
{
	FMassZoneGraphAnnotationVariableTickChunkFragment& ChunkFrag = Context.GetMutableChunkFragment<FMassZoneGraphAnnotationVariableTickChunkFragment>();
	ChunkFrag.TimeUntilNextTick -= Context.GetDeltaTimeSeconds();
	if (!ChunkFrag.bInitialized)
	{
		const bool bOffLOD = FMassSimulationVariableTickChunkFragment::GetChunkLOD(Context) == EMassLOD::Off;
		ChunkFrag.TimeUntilNextTick = FMath::RandRange(0.0f, bOffLOD ? UE::Mass::ZoneGraphAnnotations::OffLODMaxUpdateInterval : UE::Mass::ZoneGraphAnnotations::MaxUpdateInterval);
		ChunkFrag.bInitialized = true;
	}
	else
	{
		ChunkFrag.TimeUntilNextTick -= Context.GetDeltaTimeSeconds();
	}
	
	if (ChunkFrag.TimeUntilNextTick <= 0.0f)
	{
		const bool bOffLOD = FMassSimulationVariableTickChunkFragment::GetChunkLOD(Context) == EMassLOD::Off;
		ChunkFrag.TimeUntilNextTick = FMath::RandRange(
			bOffLOD ? UE::Mass::ZoneGraphAnnotations::OffLODMinUpdateInterval : UE::Mass::ZoneGraphAnnotations::MinUpdateInterval, 
			bOffLOD ? UE::Mass::ZoneGraphAnnotations::OffLODMaxUpdateInterval : UE::Mass::ZoneGraphAnnotations::MaxUpdateInterval);

		return true;
	}

	return false;
}
