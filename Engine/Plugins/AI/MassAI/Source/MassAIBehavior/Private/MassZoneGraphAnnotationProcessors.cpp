// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassZoneGraphAnnotationProcessors.h"
#include "MassAIBehaviorTypes.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassZoneGraphAnnotationFragments.h"
#include "MassZoneGraphAnnotationTypes.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "MassSignalSubsystem.h"
#include "MassSimulationLOD.h"
#include "MassExecutionContext.h"
#include "Engine/World.h"

//----------------------------------------------------------------------//
//  UMassZoneGraphAnnotationTagsInitializer
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphAnnotationProcessors)
UMassZoneGraphAnnotationTagsInitializer::UMassZoneGraphAnnotationTagsInitializer()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassZoneGraphAnnotationFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
}

void UMassZoneGraphAnnotationTagsInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassZoneGraphAnnotationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddSubsystemRequirement<UZoneGraphAnnotationSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UMassZoneGraphAnnotationTagsInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const UZoneGraphAnnotationSubsystem& ZoneGraphAnnotationSubsystem = Context.GetSubsystemChecked<UZoneGraphAnnotationSubsystem>();

		const TArrayView<FMassZoneGraphAnnotationFragment> AnnotationTagsList = Context.GetMutableFragmentView<FMassZoneGraphAnnotationFragment>();
		const TConstArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList = Context.GetFragmentView<FMassZoneGraphLaneLocationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassZoneGraphAnnotationFragment& AnnotationTags = AnnotationTagsList[EntityIt];
			const FMassZoneGraphLaneLocationFragment& LaneLocation = LaneLocationList[EntityIt];

			if (!LaneLocation.LaneHandle.IsValid())
			{
				AnnotationTags.Tags = FZoneGraphTagMask::None;
			}
			else
			{
				AnnotationTags.Tags = ZoneGraphAnnotationSubsystem.GetAnnotationTags(LaneLocation.LaneHandle);
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassZoneGraphAnnotationTagUpdateProcessor
//----------------------------------------------------------------------//
UMassZoneGraphAnnotationTagUpdateProcessor::UMassZoneGraphAnnotationTagUpdateProcessor()
{
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::UpdateAnnotationTags;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::SyncWorldToMass);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Behavior);
}

void UMassZoneGraphAnnotationTagUpdateProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);

	UMassSignalSubsystem* SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(Owner.GetWorld());
	if (ensureMsgf(SignalSubsystem, TEXT("Expecting to run this processor with UMassSignalSubsystem available")))
	{
		SubscribeToSignal(*SignalSubsystem, UE::Mass::Signals::CurrentLaneChanged);
	}
}

void UMassZoneGraphAnnotationTagUpdateProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::ConfigureQueries(EntityManager);
	EntityQuery.AddRequirement<FMassZoneGraphAnnotationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddChunkRequirement<FMassZoneGraphAnnotationVariableTickChunkFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.AddSubsystemRequirement<UZoneGraphAnnotationSubsystem>(EMassFragmentAccess::ReadWrite);

	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassZoneGraphAnnotationTagUpdateProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	TransientEntitiesToSignal.Reset();

	// Calling super will update the signals, and call SignalEntities() below.
	Super::Execute(EntityManager, Context);

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		// Periodically update tags.
		if (!FMassZoneGraphAnnotationVariableTickChunkFragment::UpdateChunk(Context))
		{
			return;
		}
		
		UZoneGraphAnnotationSubsystem& ZoneGraphAnnotationSubsystem = Context.GetMutableSubsystemChecked<UZoneGraphAnnotationSubsystem>();

		const TArrayView<FMassZoneGraphAnnotationFragment> AnnotationTagsList = Context.GetMutableFragmentView<FMassZoneGraphAnnotationFragment>();
		const TConstArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList = Context.GetFragmentView<FMassZoneGraphLaneLocationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassZoneGraphAnnotationFragment& AnnotationTags = AnnotationTagsList[EntityIt];
			const FMassZoneGraphLaneLocationFragment& LaneLocation = LaneLocationList[EntityIt];

			UpdateAnnotationTags(ZoneGraphAnnotationSubsystem, AnnotationTags, LaneLocation, Context.GetEntity(EntityIt));
		}
	});

	if (TransientEntitiesToSignal.Num())
	{
		UMassSignalSubsystem& SignalSubsystem = Context.GetMutableSubsystemChecked<UMassSignalSubsystem>();
		SignalSubsystem.SignalEntities(UE::Mass::Signals::AnnotationTagsChanged, TransientEntitiesToSignal);
	}
}

void UMassZoneGraphAnnotationTagUpdateProcessor::UpdateAnnotationTags(UZoneGraphAnnotationSubsystem& ZoneGraphAnnotationSubsystem, FMassZoneGraphAnnotationFragment& AnnotationTags, const FMassZoneGraphLaneLocationFragment& LaneLocation, const FMassEntityHandle Entity)
{
	const FZoneGraphTagMask OldTags = AnnotationTags.Tags;

	if (LaneLocation.LaneHandle.IsValid())
	{
		AnnotationTags.Tags = ZoneGraphAnnotationSubsystem.GetAnnotationTags(LaneLocation.LaneHandle);
	}
	else
	{
		AnnotationTags.Tags = FZoneGraphTagMask::None;
	}

	if (OldTags != AnnotationTags.Tags)
	{
		TransientEntitiesToSignal.Add(Entity);
	}
}

void UMassZoneGraphAnnotationTagUpdateProcessor::SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup&)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		UZoneGraphAnnotationSubsystem& ZoneGraphAnnotationSubsystem = Context.GetMutableSubsystemChecked<UZoneGraphAnnotationSubsystem>();

		const TConstArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList = Context.GetFragmentView<FMassZoneGraphLaneLocationFragment>();
		const TArrayView<FMassZoneGraphAnnotationFragment> AnnotationTagsList = Context.GetMutableFragmentView<FMassZoneGraphAnnotationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassZoneGraphAnnotationFragment& AnnotationTags = AnnotationTagsList[EntityIt];
			const FMassZoneGraphLaneLocationFragment& LaneLocation = LaneLocationList[EntityIt];

			UpdateAnnotationTags(ZoneGraphAnnotationSubsystem, AnnotationTags, LaneLocation, Context.GetEntity(EntityIt));
		}
	});
}
