// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassZoneGraphAnnotationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "Engine/World.h"
#include "MassZoneGraphAnnotationFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphAnnotationTrait)

void UMassZoneGraphAnnotationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.AddFragment<FMassZoneGraphAnnotationFragment>();
	BuildContext.AddChunkFragment<FMassZoneGraphAnnotationVariableTickChunkFragment>();
}
