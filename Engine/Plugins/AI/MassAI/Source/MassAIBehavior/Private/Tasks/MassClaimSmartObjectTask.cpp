// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassClaimSmartObjectTask.h"
#include "Engine/World.h"
#include "MassAIBehaviorTypes.h"
#include "MassCommonFragments.h"
#include "MassNavigationFragments.h"
#include "MassSignalSubsystem.h"
#include "MassSmartObjectFragments.h"
#include "MassSmartObjectHandler.h"
#include "MassSmartObjectSettings.h"
#include "MassStateTreeDependency.h"
#include "MassStateTreeExecutionContext.h"
#include "SmartObjectSubsystem.h"
#include "StateTreeLinker.h"

//----------------------------------------------------------------------//
// FMassClaimSmartObjectTask
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassClaimSmartObjectTask)

FMassClaimSmartObjectTask::FMassClaimSmartObjectTask()
{
	// This task should not react to Enter/ExitState when the state is reselected.
	bShouldStateChangeOnReselect = false;
}

bool FMassClaimSmartObjectTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(SmartObjectUserHandle);
	Linker.LinkExternalData(SmartObjectMRUSlotsHandle);
	Linker.LinkExternalData(SmartObjectSubsystemHandle);
	Linker.LinkExternalData(MassSignalSubsystemHandle);

	return true;
}

void FMassClaimSmartObjectTask::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadWrite(SmartObjectUserHandle);
	Builder.AddReadWrite(SmartObjectMRUSlotsHandle);
	// @todo SmartObjectSubsystemHandle is being used in a RW fashion, but we need this task to be able to run in parallel with
	// everything else, so we need to ensure TMassExternalSubsystemTraits<USmartObjectSubsystem> is marked up for parallel access
	// and that this information is properly utilized during Mass processing graph creation
	Builder.AddReadOnly(SmartObjectSubsystemHandle);
	Builder.AddReadOnly(MassSignalSubsystemHandle);
}

EStateTreeRunStatus FMassClaimSmartObjectTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	// Retrieve fragments and subsystems
	USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
	UMassSignalSubsystem& SignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	const FMassSmartObjectCandidateSlots* CandidateSlots = InstanceData.CandidateSlots.GetMutablePtr<FMassSmartObjectCandidateSlots>(Context);
	if (CandidateSlots == nullptr)
	{
		MASSBEHAVIOR_LOG(Log, TEXT("Candidate slots not set"));
		return EStateTreeRunStatus::Failed;
	}

	if (CandidateSlots->NumSlots == 0)
	{
		MASSBEHAVIOR_LOG(Log, TEXT("No candidate slots"));
		return EStateTreeRunStatus::Failed;
	}

	InstanceData.ClaimedSlot.Invalidate();
	InstanceData.ClaimedSlotTransform = FTransform::Identity;

	// Setup MassSmartObject handler and claim
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	const FMassSmartObjectHandler MassSmartObjectHandler(MassStateTreeContext.GetMassEntityExecutionContext(), SmartObjectSubsystem, SignalSubsystem);

	InstanceData.ClaimedSlot = MassSmartObjectHandler.ClaimCandidate(MassStateTreeContext.GetEntity(), SOUser, *CandidateSlots);

	// Treat claiming a slot as consuming all the candidate slots.
	// This is done here because of the limited ways we can communicate between FindSmartObject() and ClaimSmartObject().
	// InteractionCooldownEndTime is used by the FindSmartObject() to invalidate the candidates.
	SOUser.InteractionCooldownEndTime = Context.GetWorld()->GetTimeSeconds() + InteractionCooldown;

	if (!InstanceData.ClaimedSlot.IsValid())
	{
		MASSBEHAVIOR_LOG(Log, TEXT("Failed to claim smart object slot from %d candidates"), CandidateSlots->NumSlots);
		return EStateTreeRunStatus::Failed;
	}

	InstanceData.ClaimedSlotTransform = SmartObjectSubsystem.GetSlotTransform(InstanceData.ClaimedSlot).Get(FTransform::Identity);
	return EStateTreeRunStatus::Running;
}

void FMassClaimSmartObjectTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	// Succeeded or not, prevent interactions for a specified duration.
	SOUser.InteractionCooldownEndTime = Context.GetWorld()->GetTimeSeconds() + InteractionCooldown;

	if (InstanceData.ClaimedSlot.IsValid())
	{
		const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
		FMassExecutionContext& MassContext = MassStateTreeContext.GetMassEntityExecutionContext();
		const FMassEntityHandle Entity = MassStateTreeContext.GetEntity();

		if (UE::Mass::SmartObject::FMRUSlotsFragment* ExistingMRUSlotsFragment = Context.GetExternalDataPtr(SmartObjectMRUSlotsHandle))
		{
			ExistingMRUSlotsFragment->Slots.Push(InstanceData.ClaimedSlot.SlotHandle);
		}
		else
		{
			const UMassSmartObjectSettings* MassSmartObjectSettings = GetDefault<UMassSmartObjectSettings>();
			if (MassSmartObjectSettings != nullptr
				&& MassSmartObjectSettings->MRUSlotsMaxCount > 0)
			{
				UE::Mass::SmartObject::FMRUSlotsFragment MRUSlotsFragment;
				MRUSlotsFragment.Slots.Push(InstanceData.ClaimedSlot.SlotHandle);
				MassContext.Defer().PushCommand<FMassCommandAddFragmentInstances>(Entity, MRUSlotsFragment);
			}
		}

		USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
		UMassSignalSubsystem& SignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
		const FMassSmartObjectHandler MassSmartObjectHandler(MassContext, SmartObjectSubsystem, SignalSubsystem);

		MassSmartObjectHandler.ReleaseSmartObject(Entity, SOUser, InstanceData.ClaimedSlot);
	}
	else
	{
		MASSBEHAVIOR_LOG(VeryVerbose, TEXT("Exiting state with an invalid ClaimHandle: nothing to do."));
	}
}

EStateTreeRunStatus FMassClaimSmartObjectTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	// Prevent FindSmartObject() to query new objects while claimed.
	// This is done here because of the limited ways we can communicate between FindSmartObject() and ClaimSmartObject().
	// InteractionCooldownEndTime is used by the FindSmartObject() to invalidate the candidates.
	SOUser.InteractionCooldownEndTime = Context.GetWorld()->GetTimeSeconds() + InteractionCooldown;

	// Check that the claimed slot is still valid, and if not, fail the task.
	// The slot can become invalid if the whole SO or slot becomes invalidated.
	// In this case, we don't consider the slot as used and don't update the MRU slots
	if (InstanceData.ClaimedSlot.IsValid())
	{
		const USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
		if (!SmartObjectSubsystem.IsClaimedSmartObjectValid(InstanceData.ClaimedSlot))
		{
			InstanceData.ClaimedSlot.Invalidate();
			InstanceData.ClaimedSlotTransform = FTransform::Identity;
		}
	}

	return InstanceData.ClaimedSlot.IsValid() ? EStateTreeRunStatus::Running : EStateTreeRunStatus::Failed;
}
