// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassFindSmartObjectTargetTask.h"

#include "MassAIBehaviorTypes.h"
#include "MassCommonFragments.h"
#include "MassSmartObjectFragments.h"
#include "MassStateTreeExecutionContext.h"
#include "NavigationSystem.h"
#include "SmartObjectSubsystem.h"
#include "StateTreeLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassFindSmartObjectTargetTask)

bool FMassFindSmartObjectTargetTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(TransformHandle);
	Linker.LinkExternalData(AgentRadiusHandle);
	Linker.LinkExternalData(AgentHeightHandle);
	Linker.LinkExternalData(SmartObjectSubsystemHandle);

	return true;
}

EStateTreeRunStatus FMassFindSmartObjectTargetTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	InstanceData.SmartObjectLocation.Reset();

	if (!InstanceData.ClaimedSlot.SmartObjectHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Invalid claimed smart object ID."));
		return EStateTreeRunStatus::Failed;
	}

	const FVector EntityLocation = Context.GetExternalData(TransformHandle).GetTransform().GetLocation();
	const FAgentRadiusFragment* AgentRadius = Context.GetExternalDataPtr(AgentRadiusHandle);
	const FAgentHeightFragment* AgentHeight = Context.GetExternalDataPtr(AgentHeightHandle);

	// Try to find an entrance first
	if (bUseEntranceLocationRequest)
	{
		FSmartObjectSlotEntranceLocationRequest Request = InstanceData.EntranceRequest;

		if (Request.bProjectNavigationLocation)
		{
			if (AgentRadius != nullptr && AgentHeight != nullptr)
			{
				const FNavAgentProperties& NavAgentProperties = FNavAgentProperties(AgentRadius->Radius, AgentHeight->Height);
				const UNavigationSystemV1* NavMeshSubsystem = Cast<UNavigationSystemV1>(Context.GetWorld()->GetNavigationSystem());
				const ANavigationData* NavData = nullptr;
				if (NavMeshSubsystem)
				{
					NavData = NavMeshSubsystem->GetNavDataForProps(NavAgentProperties, EntityLocation);
				}

				if (NavMeshSubsystem && NavData)
				{
					Request.NavigationData = NavData;
				}
				else
				{
					MASSBEHAVIOR_LOG(Error, TEXT("Entrance request can not be executed since no navigation data can be found and is required for 'ProjectNavigationLocation'"));
					return EStateTreeRunStatus::Failed;
				}
			}
			else
			{
				MASSBEHAVIOR_LOG(Error, TEXT("Entrance request can not be executed since the required AgentRadius and AgentHeight fragment for 'ProjectNavigationLocation' is missing"));
				return EStateTreeRunStatus::Failed;
			}
		}

		if (InstanceData.bUseEntityLocationAsSearchLocation)
		{
			Request.SearchLocation = EntityLocation;
		}

		FSmartObjectSlotEntranceLocationResult EntryLocation;
		if (SmartObjectSubsystem.FindEntranceLocationForSlot(InstanceData.ClaimedSlot.SlotHandle, Request, EntryLocation))
		{
			InstanceData.SmartObjectLocation.EndOfPathIntent = EMassMovementAction::Stand;
			InstanceData.SmartObjectLocation.EndOfPathPosition = EntryLocation.Location;
			InstanceData.SmartObjectLocation.EndOfPathRotation = EntryLocation.Rotation.Quaternion();
			return EStateTreeRunStatus::Running;
		}
	}

	// Use slot location (can also be the fallback when no entrance location request is used and no entrances were found)
	const FTransform Transform = SmartObjectSubsystem.GetSlotTransform(InstanceData.ClaimedSlot).Get(FTransform::Identity);

	InstanceData.SmartObjectLocation.EndOfPathIntent = EMassMovementAction::Stand;
	InstanceData.SmartObjectLocation.EndOfPathPosition = Transform.GetLocation();
	InstanceData.SmartObjectLocation.EndOfPathRotation = Transform.GetRotation();

	return EStateTreeRunStatus::Running;
}
