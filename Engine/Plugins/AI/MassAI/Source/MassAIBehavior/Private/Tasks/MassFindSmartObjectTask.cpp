// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassFindSmartObjectTask.h"
#include "MassAIBehaviorTypes.h"
#include "MassCommonFragments.h"
#include "MassSignalSubsystem.h"
#include "MassSmartObjectFragments.h"
#include "MassSmartObjectHandler.h"
#include "MassStateTreeExecutionContext.h"
#include "MassZoneGraphNavigationFragments.h"
#include "SmartObjectSubsystem.h"
#include "SmartObjectZoneAnnotations.h"
#include "StateTreeLinker.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassFindSmartObjectTask)

FMassFindSmartObjectTask::FMassFindSmartObjectTask()
{
	// Do not clear the request on sustained transitions.
	// A child state (move) task can succeed on the same tick as the request is made (very likely in event based ticking).
	// That will cause transitions which would kill out request immediately.
	bShouldStateChangeOnReselect = false;
}

bool FMassFindSmartObjectTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(SmartObjectSubsystemHandle);
	Linker.LinkExternalData(MassSignalSubsystemHandle);
	Linker.LinkExternalData(EntityTransformHandle);
	Linker.LinkExternalData(SmartObjectUserHandle);
	Linker.LinkExternalData(SmartObjectMRUSlotsHandle);
	Linker.LinkExternalData(LocationHandle);

	return true;
}

void FMassFindSmartObjectTask::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	// note that we're marking this dependency as ReadOnly, even though the subsystem itself is being fetched in
	// the RW mode. No RW operations are performed, however. This comment is here to serve as reminder, that
	// if the "no rw operations being performed" state changes the call below needs to change to `Builder.AddReadWrite`
	Builder.AddReadOnly(SmartObjectSubsystemHandle);
	Builder.AddReadWrite(MassSignalSubsystemHandle);
	Builder.AddReadOnly(EntityTransformHandle);
	Builder.AddReadWrite(SmartObjectUserHandle);
	Builder.AddReadOnly(SmartObjectMRUSlotsHandle);
	Builder.AddReadOnly(LocationHandle);
}

void FMassFindSmartObjectTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	// Stop any request that are still in flight.
	if (InstanceData.SearchRequestID.IsSet())
	{
		const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
		USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
		UMassSignalSubsystem& SignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
		const FMassSmartObjectHandler MassSmartObjectHandler(
			MassStateTreeContext.GetMassEntityExecutionContext(),
			SmartObjectSubsystem,
			SignalSubsystem);
		MassSmartObjectHandler.RemoveRequest(InstanceData.SearchRequestID);
		InstanceData.SearchRequestID.Reset();

		MASSBEHAVIOR_LOG(Verbose, TEXT("Cancelling pending SmartObject search on ExitState."));
	}
}

void FMassFindSmartObjectTask::StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const
{
	const UWorld* World = Context.GetWorld();
	
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);

	// This is done here because of the limited ways we can communicate between FindSmartObject() and ClaimSmartObject().
	// ClaimSmartObject() sets the InteractionCooldownEndTime when it tries to claim the candidates.
	// Use that to signal that the candidates have been consumed (either in success or failure).
	// Doing the reset here, allows the conditions relying on bHasCandidateSlots to function properly
	// in failure cases (i.e. don't try to use failed slot).
	// This code assumes that ClaimSmartObject() accesses the candidates only in EnterState() and Tick().
	if (SOUser.InteractionHandle.IsValid() || SOUser.InteractionCooldownEndTime > World->GetTimeSeconds())
	{
		MASSBEHAVIOR_LOG(Verbose, TEXT("StateCompleted: Reset candidates because of interaction cooldown."));

		FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

		InstanceData.FoundCandidateSlots.Reset();
		InstanceData.bHasCandidateSlots = false;
	}
}

EStateTreeRunStatus FMassFindSmartObjectTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	const UWorld* World = Context.GetWorld();
	USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
	UMassSignalSubsystem& SignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	const FMassSmartObjectHandler MassSmartObjectHandler(
		MassStateTreeContext.GetMassEntityExecutionContext(),
		SmartObjectSubsystem,
		SignalSubsystem);

	using namespace UE::Mass::SmartObject;
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);
	FMRUSlotsFragment* MRUSlots = Context.GetExternalDataPtr(SmartObjectMRUSlotsHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	
	// Try to search for new slots if not already in progress.
	if (!InstanceData.SearchRequestID.IsSet())
	{
		// If the user is already using a SmartObject, or has used interaction recently, skip search and empty results. 
		if (SOUser.InteractionHandle.IsValid() || SOUser.InteractionCooldownEndTime > World->GetTimeSeconds())
		{
			MASSBEHAVIOR_LOG(Verbose, TEXT("Skipped: Recently interacted (%s %.1f)"),
				SOUser.InteractionHandle.IsValid() ? TEXT("Interacting") : TEXT("Cooldown"), FMath::Max(0.0f, SOUser.InteractionCooldownEndTime - World->GetTimeSeconds()));

			// Do not offer any new candidates during cool down.
			InstanceData.FoundCandidateSlots.Reset();
			InstanceData.bHasCandidateSlots = false;

			return EStateTreeRunStatus::Running;
		}

		// Check to see if we should request. 
		const FMassZoneGraphLaneLocationFragment* LaneLocation = Context.GetExternalDataPtr(LocationHandle);
		const bool bLaneHasChanged = (LaneLocation && InstanceData.LastLane != LaneLocation->LaneHandle);
		const bool bTimeForNextUpdate = World->GetTimeSeconds() > InstanceData.NextUpdate;

		if (bTimeForNextUpdate || bLaneHasChanged)
		{
			// Use lanes if possible for faster queries using zone graph annotations
			const FMassEntityHandle RequestingEntity = MassStateTreeContext.GetEntity();
			if (bFindFromLaneLocation && LaneLocation != nullptr)
			{
				MASSBEHAVIOR_CLOG(!LaneLocation->LaneHandle.IsValid(), Error, TEXT("Always expecting a valid lane from the ZoneGraph movement"));
				if (LaneLocation->LaneHandle.IsValid())
				{
					MASSBEHAVIOR_LOG(Log, TEXT("Requesting search candidates from lane %s (%s/%s)"),
						*LaneLocation->LaneHandle.ToString(),
						*LexToString(LaneLocation->DistanceAlongLane),
						*LexToString(LaneLocation->LaneLength));

					InstanceData.LastLane = LaneLocation->LaneHandle;

					FFindCandidatesParameters Parameters
					{
						.UserTags = SOUser.UserTags,
						.ActivityRequirements = ActivityRequirements,
						.LaneLocation = { LaneLocation->LaneHandle, LaneLocation->DistanceAlongLane },
						.MRUSlots = MRUSlots != nullptr ? MRUSlots->Slots : FMRUSlots{}
					};

					InstanceData.SearchRequestID = MassSmartObjectHandler.FindCandidatesAsync(RequestingEntity, MoveTemp(Parameters));
				}
			}
			else
			{
				FFindCandidatesParameters Parameters
				{
					.UserTags = SOUser.UserTags,
					.ActivityRequirements = ActivityRequirements,
					.Location = Context.GetExternalData(EntityTransformHandle).GetTransform().GetLocation(),
					.MRUSlots = MRUSlots != nullptr ? MRUSlots->Slots : FMRUSlots{}
				};

				InstanceData.SearchRequestID = MassSmartObjectHandler.FindCandidatesAsync(RequestingEntity, MoveTemp(Parameters));
			}
		}
	}
	else
	{
		// Poll to see if the candidates are ready.
		// A "candidates ready" signal will trigger the state tree evaluation when candidates are ready.
		if (const FMassSmartObjectCandidateSlots* NewCandidates = MassSmartObjectHandler.GetRequestCandidates(InstanceData.SearchRequestID))
		{
			MASSBEHAVIOR_LOG(Log, TEXT("Found %d smart object candidates"), NewCandidates->NumSlots);

			InstanceData.FoundCandidateSlots = *NewCandidates;
			InstanceData.bHasCandidateSlots = InstanceData.FoundCandidateSlots.NumSlots > 0;
			
			// Remove requests
			MassSmartObjectHandler.RemoveRequest(InstanceData.SearchRequestID);
			InstanceData.SearchRequestID.Reset();

			// Schedule next update.
			const FMassEntityHandle Entity = MassStateTreeContext.GetEntity();

			constexpr float SearchIntervalDeviation = 0.1f;
			const float DelayInSeconds = SearchInterval * FMath::FRandRange(1.0f - SearchIntervalDeviation, 1.0f + SearchIntervalDeviation);
				
			InstanceData.NextUpdate = World->GetTimeSeconds() + DelayInSeconds;
			SignalSubsystem.DelaySignalEntityDeferred(MassStateTreeContext.GetMassEntityExecutionContext(), UE::Mass::Signals::SmartObjectRequestCandidates, Entity, DelayInSeconds);
		}
	}
	
	return EStateTreeRunStatus::Running;
}
