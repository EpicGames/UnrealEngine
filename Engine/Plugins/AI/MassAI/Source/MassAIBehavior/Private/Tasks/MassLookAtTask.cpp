// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassLookAtTask.h"
#include "MassAIBehaviorTypes.h"
#include "MassLookAtFragments.h"
#include "MassSignalSubsystem.h"
#include "MassStateTreeExecutionContext.h"
#include "StateTreeLinker.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassLookAtTask)

bool FMassLookAtTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(MassSignalSubsystemHandle);
	Linker.LinkExternalData(LookAtHandle);
	
	return true;
}

void FMassLookAtTask::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadWrite<FMassLookAtFragment>();
	// @todo LookAt: this is only needed when InstanceData.Duration, but there's no way to tell at this point.
	Builder.AddReadWrite<UMassSignalSubsystem>();
}

bool FMassLookAtTask::TryActivateSystemicLookAt(const FStateTreeExecutionContext& Context, const FInstanceDataType& InstanceData, FMassLookAtFragment& Fragment) const
{
	// We can activate systemic LookAt in the following cases:
	// - nothing is currently active
	// - a systemic one is active (the leaf task has priority preserving the original implementation)
	// - the last override got removed while a systemic task is still running
	if (Fragment.OverrideState == FMassLookAtFragment::EOverrideState::AllDisabled
		|| Fragment.OverrideState == FMassLookAtFragment::EOverrideState::ActiveSystemicOnly
		|| Fragment.OverrideState == FMassLookAtFragment::EOverrideState::PendingSystemicReactivation)
	{
		Fragment.InterpolationSpeed = InterpolationSpeed;
		Fragment.CustomInterpolationSpeed = CustomInterpolationSpeed;
		Fragment.LookAtMode = LookAtMode;
		Fragment.TrackedEntity = InstanceData.TargetEntity;
		Fragment.OverrideState = FMassLookAtFragment::EOverrideState::ActiveSystemicOnly;

		// When using 'LookAtEntity' we validate the target entity to use it,
		// or we use the default 'LookForward' as fallback.
		if (LookAtMode == EMassLookAtMode::LookAtEntity)
		{
			if (!InstanceData.TargetEntity.IsSet())
			{
				MASSBEHAVIOR_LOG(Error, TEXT("Failed LookAt: invalid target entity"));
				Fragment.LookAtMode = EMassLookAtMode::LookForward;
				Fragment.TrackedEntity = {};
			}
		}
		return true;
	}

	return false;
}

EStateTreeRunStatus FMassLookAtTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	
	InstanceData.Time = 0.f;
	
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	FMassLookAtFragment* LookAtFragment = MassStateTreeContext.GetExternalDataPtr(LookAtHandle);
	if (LookAtFragment != nullptr)
	{
		TryActivateSystemicLookAt(Context, InstanceData, *LookAtFragment);

		LookAtFragment->RandomGazeMode = RandomGazeMode;
		LookAtFragment->RandomGazeYawVariation = RandomGazeYawVariation;
		LookAtFragment->RandomGazePitchVariation = RandomGazePitchVariation;
		LookAtFragment->bRandomGazeEntities = bRandomGazeEntities;

		// A Duration <= 0 indicates that the task runs until a transition in the state tree stops it.
		// Otherwise we schedule a signal to end the task.
		if (InstanceData.Duration > 0.0f)
		{
			UMassSignalSubsystem& MassSignalSubsystem = MassStateTreeContext.GetExternalData(MassSignalSubsystemHandle);
			MassSignalSubsystem.DelaySignalEntityDeferred(MassStateTreeContext.GetMassEntityExecutionContext(), UE::Mass::Signals::LookAtFinished, MassStateTreeContext.GetEntity(), InstanceData.Duration);
		}
	}

	// LookAt are considered optional so we are returning an immediate success when fragment is missing and a specific duration was specified.
	if (LookAtFragment == nullptr && InstanceData.Duration > 0.0f)
	{
		return EStateTreeRunStatus::Succeeded;
	}

	return EStateTreeRunStatus::Running;
}

void FMassLookAtTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);

	if (FMassLookAtFragment* LookAtFragment = MassStateTreeContext.GetExternalDataPtr(LookAtHandle))
	{
		LookAtFragment->ResetSystemicLookAt();
	}
}

EStateTreeRunStatus FMassLookAtTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	InstanceData.Time += DeltaTime;

	const EStateTreeRunStatus Status = InstanceData.Duration <= 0.0f ? EStateTreeRunStatus::Running : (InstanceData.Time < InstanceData.Duration ? EStateTreeRunStatus::Running : EStateTreeRunStatus::Succeeded);

	// We might get notified that a LookAt override finished so we can try to activate again
	if (Status == EStateTreeRunStatus::Running)
	{
		const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
		if (FMassLookAtFragment* Fragment = MassStateTreeContext.GetExternalDataPtr(LookAtHandle); Fragment != nullptr)
		{
			// Only care about reactivation in the Tick
			if (Fragment->OverrideState == FMassLookAtFragment::EOverrideState::PendingSystemicReactivation)
			{
				TryActivateSystemicLookAt(Context, InstanceData, *Fragment);
			}
		}
	}

	return Status;
}
