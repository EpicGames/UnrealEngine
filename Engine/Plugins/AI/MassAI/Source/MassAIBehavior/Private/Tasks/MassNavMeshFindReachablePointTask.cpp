// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassNavMeshFindReachablePointTask.h"
#include "MassAIBehaviorTypes.h"
#include "MassCommonFragments.h"
#include "MassNavigationFragments.h"
#include "MassDebugger.h"
#include "MassStateTreeDependency.h"
#include "MassStateTreeExecutionContext.h"
#include "NavigationSystem.h"
#include "StateTreeLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavMeshFindReachablePointTask)

bool FMassNavMeshFindReachablePointTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(TransformHandle);
	Linker.LinkExternalData(AgentRadiusHandle);
	Linker.LinkExternalData(AgentHeightHandle);

	return true;
}

EStateTreeRunStatus FMassNavMeshFindReachablePointTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG

	const FAgentRadiusFragment& AgentRadius = Context.GetExternalData(AgentRadiusHandle);
	const FAgentHeightFragment& AgentHeight = Context.GetExternalData(AgentHeightHandle);
	const FVector AgentNavLocation = Context.GetExternalData(TransformHandle).GetTransform().GetLocation();
	const FNavAgentProperties& NavAgentProperties = FNavAgentProperties(AgentRadius.Radius, AgentHeight.Height);

	UNavigationSystemV1* NavMeshSubsystem = Cast<UNavigationSystemV1>(Context.GetWorld()->GetNavigationSystem());
	const ANavigationData* NavData = NavMeshSubsystem->GetNavDataForProps(NavAgentProperties, AgentNavLocation);

	if (NavData == nullptr)
	{
		MASSBEHAVIOR_LOG(Warning, TEXT("Invalid NavData"));
		return EStateTreeRunStatus::Failed;
	}

	FNavLocation NavLocation;
	if (!NavData->GetRandomReachablePointInRadius(AgentNavLocation, InstanceData.SearchRadius, NavLocation))
	{
		MASSBEHAVIOR_LOG(Warning, TEXT("No reachable points found"));
		return EStateTreeRunStatus::Failed;
	}

	InstanceData.TargetLocation.EndOfPathPosition = NavLocation.Location;
	InstanceData.TargetLocation.EndOfPathIntent = EMassMovementAction::Move;

	if (bDisplayDebug)
	{
		MASSBEHAVIOR_LOG(Log, TEXT("Found target: '%s' at %s"), *UEnum::GetValueAsString(InstanceData.TargetLocation.EndOfPathIntent),  *NavLocation.Location.ToString());
	}

	return EStateTreeRunStatus::Running;
}
