// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassNavMeshPathfollowTask.h"

#include "MassAIBehaviorTypes.h"
#include "MassCommonFragments.h"
#include "MassDebugger.h"
#include "MassMovementFragments.h"
#include "MassNavigationFragments.h"
#include "MassStateTreeExecutionContext.h"
#include "NavCorridor.h"
#include "NavigationSystem.h"
#include "StateTreeLinker.h"

bool FMassNavMeshPathFollowTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(TransformHandle);
	Linker.LinkExternalData(MoveTargetHandle);
	Linker.LinkExternalData(AgentRadiusHandle);
	Linker.LinkExternalData(AgentHeightHandle);
	Linker.LinkExternalData(DesiredMovementHandle);
	Linker.LinkExternalData(MovementParamsHandle);
	Linker.LinkExternalData(CachedPathHandle);
	Linker.LinkExternalData(ShortPathHandle);

	return true;
}

bool FMassNavMeshPathFollowTask::RequestPath(FStateTreeExecutionContext& Context, const FMassTargetLocation& InTargetLocation) const
{
	FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	FMassNavMeshPathFollowTaskInstanceData& InstanceData = Context.GetInstanceData<FMassNavMeshPathFollowTaskInstanceData>(*this);

	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG
	
	const FAgentRadiusFragment& AgentRadius = Context.GetExternalData(AgentRadiusHandle);
	const FAgentHeightFragment& AgentHeight = Context.GetExternalData(AgentHeightHandle);
	const FVector AgentNavLocation = Context.GetExternalData(TransformHandle).GetTransform().GetLocation();
	const FNavAgentProperties& NavAgentProperties = FNavAgentProperties(AgentRadius.Radius, AgentHeight.Height);

	UNavigationSystemV1* NavMeshSubsystem = Cast<UNavigationSystemV1>(Context.GetWorld()->GetNavigationSystem());
	if (!NavMeshSubsystem)
	{
		MASSBEHAVIOR_LOG(Warning, TEXT("Missing navigation system."));
		return false;
	}
	
	const ANavigationData* NavData = NavMeshSubsystem->GetNavDataForProps(NavAgentProperties, AgentNavLocation);
	
	if (!NavData || !InTargetLocation.EndOfPathPosition.IsSet())
	{
		MASSBEHAVIOR_LOG(Warning, TEXT("%s"), !NavData ? TEXT("Invalid NavData") : TEXT("EndOfPathPosition not set") );
		return false;
	}
		
	FPathFindingQuery Query(NavMeshSubsystem, *NavData, AgentNavLocation, InTargetLocation.EndOfPathPosition.GetValue());

	// Why fix it after if there is none??
	if (!Query.NavData.IsValid())
	{
		Query.NavData = NavMeshSubsystem->GetNavDataForProps(NavAgentProperties, Query.StartLocation);
	}

	FPathFindingResult Result(ENavigationQueryResult::Error);
	if (Query.NavData.IsValid())
	{
		if (bDisplayDebug)
		{
			MASSBEHAVIOR_LOG(Verbose, TEXT("requesting synchronous path"));
		}
		
		Result = Query.NavData->FindPath(NavAgentProperties, Query);
	}

	if (Result.IsSuccessful())
	{
		Result.Path->RemoveOverlappingPoints(FNavCorridor::OverlappingPointTolerance);
		
		// @todo: Investigate single point paths but for now, only move if we have more than one point.
		if (Result.Path.Get()->GetPathPoints().Num() > 1)
		{
			// A path was found
			if (bDisplayDebug)
			{
				MASSBEHAVIOR_LOG(Verbose, TEXT("path found"));
			}

			FMassNavMeshCachedPathFragment& CachedPathFragment = Context.GetExternalData(CachedPathHandle);
			CachedPathFragment.NavPath = Result.Path;

			CachedPathFragment.PathSource = EMassNavigationPathSource::NavMesh;

			// Build corridor
			CachedPathFragment.Corridor = MakeShared<FNavCorridor>();
			const FSharedConstNavQueryFilter NavQueryFilter = Query.QueryFilter ? Query.QueryFilter : NavData->GetDefaultQueryFilter();

			FNavCorridorParams CorridorParams;
			CorridorParams.SetFromWidth(InstanceData.CorridorWidth);
			CorridorParams.PathOffsetFromBoundaries = InstanceData.OffsetFromBoundaries;

			MASSBEHAVIOR_LOG(Verbose, TEXT("FMassNavMeshPathFollowTask::RequestPath"));
			MASSBEHAVIOR_LOG(Verbose, TEXT("Start: %s, End: %s"),
				*CachedPathFragment.NavPath->GetPathPoints()[0].Location.ToCompactString(),
				*CachedPathFragment.NavPath->GetPathPoints().Last().Location.ToCompactString());
			MASSBEHAVIOR_LOG(Verbose, TEXT("Corridor params - %s"), *CorridorParams.ToString());
			
			CachedPathFragment.Corridor->BuildFromPath(*CachedPathFragment.NavPath, NavQueryFilter, CorridorParams);

			if (InstanceData.OffsetCornersFromBoundaries > 0.f)
			{
				CachedPathFragment.Corridor->OffsetPathLocationsFromWalls(InstanceData.OffsetCornersFromBoundaries);
			}

			// Update short path
			FMassNavMeshShortPathFragment& ShortPathFragment = Context.GetExternalData(ShortPathHandle);
			ShortPathFragment.RequestShortPath(CachedPathFragment.Corridor, /*NavCorridorStartIndex*/0, /*NumLeadingPoints*/0, InstanceData.EndDistanceThreshold);

			CachedPathFragment.NavPathNextStartIndex = (uint16)FMath::Max(ShortPathFragment.NumPoints - FMassNavMeshShortPathFragment::NumPointsBeyondUpdate - FMassNavMeshCachedPathFragment::NumLeadingPoints, 0);
			
			// Update MoveTarget
			FMassMoveTargetFragment& MoveTarget = Context.GetExternalData(MoveTargetHandle);
			const FMassMovementParameters& MovementParams = Context.GetExternalData(MovementParamsHandle);
			float DesiredSpeed = FMath::Min(
				MovementParams.GenerateDesiredSpeed(InstanceData.MovementStyle, MassContext.GetEntity().Index) * InstanceData.SpeedScale,
				MovementParams.MaxSpeed);

			// Apply DesiredMaxSpeedOverride
			const FMassDesiredMovementFragment& DesiredMovementFragment = Context.GetExternalData(DesiredMovementHandle);
			DesiredSpeed = FMath::Min(DesiredSpeed, DesiredMovementFragment.DesiredMaxSpeedOverride);

			MoveTarget.DesiredSpeed.Set(DesiredSpeed);
			
			MoveTarget.CreateNewAction(EMassMovementAction::Move, *Context.GetWorld());

			return true;
		}
	
		return true;
	}

	MASSBEHAVIOR_LOG(Warning, TEXT("Failed to find a path, result: %s (start: %s, end: %s"),
		 *UEnum::GetValueAsString(Result.Result), *Query.StartLocation.ToCompactString(), *Query.EndLocation.ToCompactString());
	
	return false;
}

bool FMassNavMeshPathFollowTask::UpdateShortPath(FStateTreeExecutionContext& Context) const
{
	FMassNavMeshPathFollowTaskInstanceData& InstanceData = Context.GetInstanceData<FMassNavMeshPathFollowTaskInstanceData>(*this);
	
	FMassNavMeshCachedPathFragment& CachedPathFragment = Context.GetExternalData(CachedPathHandle);
	MASSBEHAVIOR_LOG(Verbose, TEXT("updating short path, starting at index %i"), CachedPathFragment.NavPathNextStartIndex);
	
	FMassNavMeshShortPathFragment& ShortPathFragment = Context.GetExternalData(ShortPathHandle);

	ShortPathFragment.RequestShortPath(CachedPathFragment.Corridor, CachedPathFragment.NavPathNextStartIndex, CachedPathFragment.NumLeadingPoints, InstanceData.EndDistanceThreshold);

	CachedPathFragment.NavPathNextStartIndex += (uint16)FMath::Max(ShortPathFragment.NumPoints - FMassNavMeshShortPathFragment::NumPointsBeyondUpdate - FMassNavMeshCachedPathFragment::NumLeadingPoints, 0);

	return true;
}

EStateTreeRunStatus FMassNavMeshPathFollowTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	FMassNavMeshPathFollowTaskInstanceData& InstanceData = Context.GetInstanceData<FMassNavMeshPathFollowTaskInstanceData>(*this);

	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG
	if (bDisplayDebug)
	{
		MASSBEHAVIOR_LOG(Verbose, TEXT("enterstate."));
	}

	if (!InstanceData.TargetLocation.EndOfPathPosition.IsSet())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Target is not defined."));
		return EStateTreeRunStatus::Failed;
	}
		
	if (!RequestPath(Context, InstanceData.TargetLocation))
	{
		MASSBEHAVIOR_LOG(Warning, TEXT("Failed to request path."));
		return EStateTreeRunStatus::Failed;
	}

	return EStateTreeRunStatus::Running;
}

EStateTreeRunStatus FMassNavMeshPathFollowTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);

	FMassNavMeshPathFollowTaskInstanceData& InstanceData = Context.GetInstanceData<FMassNavMeshPathFollowTaskInstanceData>(*this);

	FMassNavMeshShortPathFragment& ShortPathFragment = Context.GetExternalData(ShortPathHandle);
	
	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG
	if (bDisplayDebug)
	{
		MASSBEHAVIOR_LOG(Verbose, TEXT("tick"));
	}
	
	// Current path follow is done, but it was partial (i.e. many points on a curve), try again until we get there.
	if (ShortPathFragment.IsDone() && ShortPathFragment.bPartialResult)
	{
		if (!InstanceData.TargetLocation.EndOfPathPosition.IsSet())
		{
			MASSBEHAVIOR_LOG(Error, TEXT("Target is not defined."));
			return EStateTreeRunStatus::Failed;
		}

		if (!UpdateShortPath(Context))
		{
			MASSBEHAVIOR_LOG(Error, TEXT("Failed to update short path."));
			return EStateTreeRunStatus::Failed;
		}
	}
	
	return ShortPathFragment.IsDone() ? EStateTreeRunStatus::Succeeded : EStateTreeRunStatus::Running;
}
