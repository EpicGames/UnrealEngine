// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassNavMeshStandTask.h"
#include "StateTreeExecutionContext.h"
#include "MassMovementFragments.h"
#include "MassNavigationFragments.h"
#include "MassNavMeshNavigationFragments.h"
#include "MassNavMeshNavigationUtils.h"
#include "MassStateTreeExecutionContext.h"
#include "MassSignalSubsystem.h"
#include "StateTreeLinker.h"
#include "MassCommonFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavMeshStandTask)

bool FMassNavMeshStandTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(MoveTargetHandle);
	Linker.LinkExternalData(ShortPathHandle);
	Linker.LinkExternalData(MovementParamsHandle);
	Linker.LinkExternalData(MassSignalSubsystemHandle);
	Linker.LinkExternalData(TransformHandle);

	return true;
}

EStateTreeRunStatus FMassNavMeshStandTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);

	const FMassMovementParameters& MovementParams = Context.GetExternalData(MovementParamsHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	FMassNavMeshShortPathFragment& ShortPath = Context.GetExternalData(ShortPathHandle);
	FMassMoveTargetFragment& MoveTarget = Context.GetExternalData(MoveTargetHandle);

	const UWorld* World = Context.GetWorld();
	checkf(World != nullptr, TEXT("A valid world is expected from the execution context."));

	const FVector AgentNavLocation = Context.GetExternalData(TransformHandle).GetTransform().GetLocation();
	MoveTarget.Center = AgentNavLocation;
	MoveTarget.CreateNewAction(EMassMovementAction::Stand, *World);
	
	const bool bSuccess = UE::MassNavigation::ActivateActionStand(Context.GetOwner(), MassStateTreeContext.GetEntity(), 
		MovementParams.DefaultDesiredSpeed, MoveTarget, ShortPath);

	if (!bSuccess)
	{
		return EStateTreeRunStatus::Failed;
	}

	InstanceData.Time = 0.f;

	// A Duration <= 0 indicates that the task runs until a transition in the state tree stops it.
	// Otherwise, we schedule a signal to end the task.
	if (InstanceData.Duration > 0.f)
	{
		UMassSignalSubsystem& MassSignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
		MassSignalSubsystem.DelaySignalEntityDeferred(MassStateTreeContext.GetMassEntityExecutionContext(),
			UE::Mass::Signals::StandTaskFinished, MassStateTreeContext.GetEntity(), InstanceData.Duration);
	}

	return EStateTreeRunStatus::Running;
}

EStateTreeRunStatus FMassNavMeshStandTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	
	InstanceData.Time += DeltaTime;
	
	return InstanceData.Duration <= 0.f ? EStateTreeRunStatus::Running :
		(InstanceData.Time < InstanceData.Duration ? EStateTreeRunStatus::Running : EStateTreeRunStatus::Succeeded);
}
