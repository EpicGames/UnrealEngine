// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassUseSmartObjectTask.h"
#include "Engine/World.h"
#include "MassAIBehaviorTypes.h"
#include "MassNavigationFragments.h"
#include "MassSignalSubsystem.h"
#include "MassSmartObjectFragments.h"
#include "MassSmartObjectHandler.h"
#include "MassStateTreeExecutionContext.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassZoneGraphNavigationUtils.h"
#include "SmartObjectSubsystem.h"
#include "StateTreeLinker.h"
#include "MassStateTreeDependency.h"

//----------------------------------------------------------------------//
// FMassUseSmartObjectTask
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassUseSmartObjectTask)

FMassUseSmartObjectTask::FMassUseSmartObjectTask()
{
	// This task should not react to Enter/ExitState when the state is reselected.
	bShouldStateChangeOnReselect = false;
}

bool FMassUseSmartObjectTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(SmartObjectSubsystemHandle);
	Linker.LinkExternalData(MassSignalSubsystemHandle);
	Linker.LinkExternalData(SmartObjectUserHandle);
	Linker.LinkExternalData(MoveTargetHandle);

	return true;
}

void FMassUseSmartObjectTask::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadWrite(SmartObjectSubsystemHandle);
	Builder.AddReadWrite(MassSignalSubsystemHandle);
	Builder.AddReadWrite(SmartObjectUserHandle);
	Builder.AddReadWrite(MoveTargetHandle);
}

EStateTreeRunStatus FMassUseSmartObjectTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);
	
	if (SOUser.InteractionHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Agent is already using smart object slot %s."), *LexToString(SOUser.InteractionHandle));
		return EStateTreeRunStatus::Failed;
	}

	USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
	UMassSignalSubsystem& SignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
	FMassMoveTargetFragment& MoveTarget = Context.GetExternalData(MoveTargetHandle);

	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	// Setup MassSmartObject handler and start interaction
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	const FMassSmartObjectHandler MassSmartObjectHandler(MassStateTreeContext.GetMassEntityExecutionContext(), SmartObjectSubsystem, SignalSubsystem);

	if (!MassSmartObjectHandler.StartUsingSmartObject(MassStateTreeContext.GetEntity(), SOUser, InstanceData.ClaimedSlot))
	{
		return EStateTreeRunStatus::Failed;
	}

	// @todo: we should have common API to control this, currently handle via tasks.
	const UWorld* World = Context.GetWorld();
	checkf(World != nullptr, TEXT("A valid world is expected from the execution context"));

	MoveTarget.CreateNewAction(EMassMovementAction::Animate, *World);
	const bool bSuccess = UE::MassNavigation::ActivateActionAnimate(*World, Context.GetOwner(), MassStateTreeContext.GetEntity(), MoveTarget);

	return bSuccess ? EStateTreeRunStatus::Running : EStateTreeRunStatus::Failed;
}

void FMassUseSmartObjectTask::ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);

	if (SOUser.InteractionHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(VeryVerbose, TEXT("Exiting state with a valid InteractionHandle: stop using the smart object."));

		const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
		USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
		UMassSignalSubsystem& SignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
		const FMassSmartObjectHandler MassSmartObjectHandler(MassStateTreeContext.GetMassEntityExecutionContext(), SmartObjectSubsystem, SignalSubsystem);
		MassSmartObjectHandler.StopUsingSmartObject(MassStateTreeContext.GetEntity(), SOUser, EMassSmartObjectInteractionStatus::Aborted);
	}
	else
	{
		MASSBEHAVIOR_LOG(VeryVerbose, TEXT("Exiting state with an invalid ClaimHandle: nothing to do."));
	}
}

void FMassUseSmartObjectTask::StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const
{
	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);

	if (SOUser.InteractionHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(VeryVerbose, TEXT("Completing state with a valid InteractionHandle: stop using the smart object."));

		const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);
		USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
		UMassSignalSubsystem& SignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
		const FMassSmartObjectHandler MassSmartObjectHandler(MassStateTreeContext.GetMassEntityExecutionContext(), SmartObjectSubsystem, SignalSubsystem);
		const EMassSmartObjectInteractionStatus NewStatus = (CompletionStatus == EStateTreeRunStatus::Succeeded)
																? EMassSmartObjectInteractionStatus::TaskCompleted
																: EMassSmartObjectInteractionStatus::Aborted;
		MassSmartObjectHandler.StopUsingSmartObject(MassStateTreeContext.GetEntity(), SOUser, NewStatus);
	}
}

EStateTreeRunStatus FMassUseSmartObjectTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	EStateTreeRunStatus Status = EStateTreeRunStatus::Failed;

	FMassSmartObjectUserFragment& SOUser = Context.GetExternalData(SmartObjectUserHandle);
	switch (SOUser.InteractionStatus)
	{
	case EMassSmartObjectInteractionStatus::InProgress:
		MASSBEHAVIOR_LOG(VeryVerbose, TEXT("Interacting ..."));
		Status = EStateTreeRunStatus::Running;
		break;

	case EMassSmartObjectInteractionStatus::BehaviorCompleted:
		MASSBEHAVIOR_LOG(Log, TEXT("Behavior completed"));
		Status = EStateTreeRunStatus::Succeeded;
		break;

	case EMassSmartObjectInteractionStatus::TaskCompleted:
		ensureMsgf(false, TEXT("Not expecting to tick an already completed task"));
		Status = EStateTreeRunStatus::Failed;
		break;

	case EMassSmartObjectInteractionStatus::Aborted:
		MASSBEHAVIOR_LOG(Log, TEXT("Interaction aborted"));
		Status = EStateTreeRunStatus::Failed;
		break;

	case EMassSmartObjectInteractionStatus::Unset:
		MASSBEHAVIOR_LOG(Error, TEXT("Error while using smart object: interaction state is not valid"));
		Status = EStateTreeRunStatus::Failed;
		break;

	default:
		ensureMsgf(false, TEXT("Unhandled interaction status %s => Returning EStateTreeRunStatus::Failed"), *UEnum::GetValueAsString(SOUser.InteractionStatus));
		Status = EStateTreeRunStatus::Failed;
	}

	return Status;
}
