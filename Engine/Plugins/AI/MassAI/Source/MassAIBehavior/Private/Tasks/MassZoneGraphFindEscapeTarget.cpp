// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassZoneGraphFindEscapeTarget.h"
#include "StateTreeExecutionContext.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "MassZoneGraphNavigationFragments.h"
#include "Annotations/ZoneGraphDisturbanceAnnotation.h"
#include "MassAIBehaviorTypes.h"
#include "MassStateTreeExecutionContext.h"
#include "ZoneGraphSettings.h"
#include "StateTreeLinker.h"
#include "MassDebugger.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphFindEscapeTarget)


FMassZoneGraphFindEscapeTarget::FMassZoneGraphFindEscapeTarget()
{
}

bool FMassZoneGraphFindEscapeTarget::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(LocationHandle);
	Linker.LinkExternalData(ZoneGraphSubsystemHandle);
	Linker.LinkExternalData(ZoneGraphAnnotationSubsystemHandle);

	return true;
}

void FMassZoneGraphFindEscapeTarget::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly(LocationHandle);
	Builder.AddReadOnly(ZoneGraphSubsystemHandle);
	Builder.AddReadOnly(ZoneGraphAnnotationSubsystemHandle);
}

EStateTreeRunStatus FMassZoneGraphFindEscapeTarget::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	bool bDisplayDebug = false;

#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG

	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetExternalData(ZoneGraphSubsystemHandle);
	UZoneGraphAnnotationSubsystem& ZoneGraphAnnotationSubsystem = Context.GetExternalData(ZoneGraphAnnotationSubsystemHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	if (!LaneLocation.LaneHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Invalid lane handle."));
		return EStateTreeRunStatus::Failed;
	}
			
	const FZoneGraphStorage* ZoneGraphStorage = ZoneGraphSubsystem.GetZoneGraphStorage(LaneLocation.LaneHandle.DataHandle);
	if (!ZoneGraphStorage)
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Missing ZoneGraph Storage for current lane %s."), *LaneLocation.LaneHandle.ToString());
		return EStateTreeRunStatus::Failed;
	}

	const UZoneGraphDisturbanceAnnotation* DisturbanceAnnotation = Cast<const UZoneGraphDisturbanceAnnotation>(ZoneGraphAnnotationSubsystem.GetFirstAnnotationForTag(DisturbanceAnnotationTag));
	if (!DisturbanceAnnotation)
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Failed to find Flee Behavior for tag %s on lane %s."), *UE::ZoneGraph::Helpers::GetTagName(DisturbanceAnnotationTag).ToString(), *LaneLocation.LaneHandle.ToString());
		return EStateTreeRunStatus::Failed;
	}

	const FZoneGraphEscapeLaneAction* EscapeAction = DisturbanceAnnotation->GetEscapeAction(LaneLocation.LaneHandle);
	if (!EscapeAction)
	{
		MASSBEHAVIOR_LOG(Warning, TEXT("Failed to find escape action for current lane %s."), *LaneLocation.LaneHandle.ToString());
		return EStateTreeRunStatus::Failed;
	}

	const uint8 SpanIndex = EscapeAction->FindSpanIndex(LaneLocation.DistanceAlongLane);
	const FZoneGraphEscapeLaneSpan& EscapeSpan = EscapeAction->Spans[SpanIndex];
	const float MoveDir = EscapeSpan.bReverseLaneDirection ? -1.f : 1.f;

	constexpr float AdjacentMoveDistance = 50.0f;
	constexpr float MoveDistanceRandomDeviation = 250.0f;
	constexpr float BaseMoveDistance = 800.0f;
	const float MoveDistance = FMath::Max(0.0f, BaseMoveDistance + FMath::RandRange(-MoveDistanceRandomDeviation, MoveDistanceRandomDeviation));

	if (EscapeSpan.ExitLaneIndex == INDEX_NONE)
	{
		// @todo: Verbosity of this message was downgraded from 'Warning' to 'Log' until we have time to investigate.
		// MASSBEHAVIOR_LOG now outputs to the debug log in addition to the visual logger and some sample
		// projects (e.g., CitySample) might spam.
		MASSBEHAVIOR_LOG(Log, TEXT("Invalid flee exit lane."));
		return EStateTreeRunStatus::Failed;
	}
	
	if (EscapeSpan.ExitLinkType == EZoneLaneLinkType::Adjacent)
	{
		// TODO: could improve this by checking the adjacent lane type and move based on that:
		// - split: move closer to beginning of the lane
		// - merge: move closer to end of the lane
		// - adjacent: ?
		// Could also try to sample few locations along the lane to see which is closest.
		
		// Small move, and goto adjacent lane
		InstanceData.EscapeTargetLocation.LaneHandle = LaneLocation.LaneHandle;
		InstanceData.EscapeTargetLocation.TargetDistance = FMath::Clamp(LaneLocation.DistanceAlongLane + AdjacentMoveDistance * MoveDir, 0.f, LaneLocation.LaneLength);
		InstanceData.EscapeTargetLocation.NextExitLinkType = EZoneLaneLinkType::Adjacent;
		InstanceData.EscapeTargetLocation.NextLaneHandle = FZoneGraphLaneHandle(EscapeSpan.ExitLaneIndex, ZoneGraphStorage->DataHandle);
		InstanceData.EscapeTargetLocation.bMoveReverse = EscapeSpan.bReverseLaneDirection;
		InstanceData.EscapeTargetLocation.EndOfPathIntent = EMassMovementAction::Move;

		MASSBEHAVIOR_CLOG(bDisplayDebug, Log, TEXT("Switching from lane %s to adjacent lane %s."),
			*LaneLocation.LaneHandle.ToString(), *InstanceData.EscapeTargetLocation.NextLaneHandle.ToString());
	}
	else
	{
		// Forward or backwards on current lane.
		InstanceData.EscapeTargetLocation.LaneHandle = LaneLocation.LaneHandle;
		InstanceData.EscapeTargetLocation.TargetDistance = LaneLocation.DistanceAlongLane + MoveDistance * MoveDir;
		InstanceData.EscapeTargetLocation.NextExitLinkType = EZoneLaneLinkType::None;
		InstanceData.EscapeTargetLocation.NextLaneHandle.Reset();
		InstanceData.EscapeTargetLocation.bMoveReverse = EscapeSpan.bReverseLaneDirection;
		InstanceData.EscapeTargetLocation.EndOfPathIntent = EMassMovementAction::Move;

		// When close to end of a lane, choose next lane.
		const bool bPastStart = InstanceData.EscapeTargetLocation.TargetDistance < 0.0f;
		const bool bPastEnd = InstanceData.EscapeTargetLocation.TargetDistance > LaneLocation.LaneLength;
		if (bPastStart || bPastEnd)
		{
			InstanceData.EscapeTargetLocation.TargetDistance = FMath::Clamp(InstanceData.EscapeTargetLocation.TargetDistance, 0.0f, LaneLocation.LaneLength);

			InstanceData.EscapeTargetLocation.NextExitLinkType = EscapeSpan.ExitLinkType;
			InstanceData.EscapeTargetLocation.NextLaneHandle = FZoneGraphLaneHandle(EscapeSpan.ExitLaneIndex, ZoneGraphStorage->DataHandle);

			MASSBEHAVIOR_CLOG(bDisplayDebug, Log, TEXT("Advancing %s along flee lane %s to next lane %s at distance %.1f."),
				InstanceData.EscapeTargetLocation.bMoveReverse ? TEXT("forward") : TEXT("reverse"),
				*InstanceData.EscapeTargetLocation.LaneHandle.ToString(), *InstanceData.EscapeTargetLocation.NextLaneHandle.ToString(),
				InstanceData.EscapeTargetLocation.TargetDistance);
		}
		else
		{
			MASSBEHAVIOR_CLOG(bDisplayDebug, Log, TEXT("Advancing %s along flee lane %s to distance %.1f."),
				InstanceData.EscapeTargetLocation.bMoveReverse ? TEXT("forward") : TEXT("reverse"),
				*InstanceData.EscapeTargetLocation.LaneHandle.ToString(), InstanceData.EscapeTargetLocation.TargetDistance);
		}
	}

	return EStateTreeRunStatus::Running;
}
