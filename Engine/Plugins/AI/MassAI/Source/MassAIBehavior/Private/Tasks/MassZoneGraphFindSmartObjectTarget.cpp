// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassZoneGraphFindSmartObjectTarget.h"
#include "ZoneGraphAnnotationSubsystem.h"
#include "MassAIBehaviorTypes.h"
#include "SmartObjectSubsystem.h"
#include "MassSmartObjectSettings.h"
#include "MassStateTreeExecutionContext.h"
#include "MassZoneGraphNavigationFragments.h"
#include "SmartObjectZoneAnnotations.h"
#include "StateTreeLinker.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphFindSmartObjectTarget)

bool FMassZoneGraphFindSmartObjectTarget::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(LocationHandle);
	Linker.LinkExternalData(AnnotationSubsystemHandle);
	Linker.LinkExternalData(SmartObjectSubsystemHandle);
	
	return true;
}

void FMassZoneGraphFindSmartObjectTarget::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly(LocationHandle);
	Builder.AddReadOnly(AnnotationSubsystemHandle);
	Builder.AddReadOnly(SmartObjectSubsystemHandle);
}

EStateTreeRunStatus FMassZoneGraphFindSmartObjectTarget::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const UZoneGraphAnnotationSubsystem& AnnotationSubsystem = Context.GetExternalData(AnnotationSubsystemHandle);
	const USmartObjectSubsystem& SmartObjectSubsystem = Context.GetExternalData(SmartObjectSubsystemHandle);
	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	const FZoneGraphLaneHandle LaneHandle(LaneLocation.LaneHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	InstanceData.SmartObjectLocation.Reset();

	if (!InstanceData.ClaimedSlot.SmartObjectHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Invalid claimed smart object ID."));
		return EStateTreeRunStatus::Failed;
	}

	if (!LaneHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Invalid lane location."));
		return EStateTreeRunStatus::Failed;
	}

	const FZoneGraphTag SmartObjectTag = GetDefault<UMassSmartObjectSettings>()->SmartObjectTag;
	const USmartObjectZoneAnnotations* SOAnnotations = Cast<USmartObjectZoneAnnotations>(AnnotationSubsystem.GetFirstAnnotationForTag(SmartObjectTag));
	const FTransform Transform = SmartObjectSubsystem.GetSlotTransform(InstanceData.ClaimedSlot).Get(FTransform::Identity);

	InstanceData.SmartObjectLocation.LaneHandle = LaneHandle;
	InstanceData.SmartObjectLocation.NextExitLinkType = EZoneLaneLinkType::None;
	InstanceData.SmartObjectLocation.NextLaneHandle.Reset();
	InstanceData.SmartObjectLocation.bMoveReverse = false;
	InstanceData.SmartObjectLocation.EndOfPathIntent = EMassMovementAction::Stand;
	InstanceData.SmartObjectLocation.EndOfPathPosition = Transform.GetLocation();
	// Can't set direction at the moment since it seems problematic if it's opposite to the steering direction
	//SmartObjectLocation.EndOfPathDirection = SOUser.TargetDirection;

	// Let's start moving toward the interaction a bit before the entry point.
	InstanceData.SmartObjectLocation.AnticipationDistance.Set(100.f);

	// Find entry point on lane for the claimed object
	TOptional<FSmartObjectLaneLocation> SmartObjectLaneLocation;
	if (SOAnnotations != nullptr)
	{
		SmartObjectLaneLocation = SOAnnotations->GetSmartObjectLaneLocation(LaneHandle.DataHandle, InstanceData.ClaimedSlot.SmartObjectHandle);
	}

	if (SmartObjectLaneLocation.IsSet())
	{
		// Request path along current lane to reach entry point on lane
		MASSBEHAVIOR_LOG(Log, TEXT("Claim successful: create path along lane to reach interaction location."));
		InstanceData.SmartObjectLocation.TargetDistance = SmartObjectLaneLocation.GetValue().DistanceAlongLane;
	}
	else
	{
		// Request path from current lane location directly to interaction location
		MASSBEHAVIOR_LOG(Warning, TEXT("Claim successful: create path from current lane location directly to interaction location since SmartObject zone annotations weren't found."));
		InstanceData.SmartObjectLocation.TargetDistance = LaneLocation.DistanceAlongLane;
	}
	
	return EStateTreeRunStatus::Running;
}
