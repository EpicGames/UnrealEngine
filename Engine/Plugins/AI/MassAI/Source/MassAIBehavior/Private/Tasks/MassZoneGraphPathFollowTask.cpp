// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassZoneGraphPathFollowTask.h"
#include "MassAIBehaviorTypes.h"
#include "MassCommonFragments.h"
#include "MassNavigationFragments.h"
#include "MassMovementFragments.h"
#include "MassStateTreeExecutionContext.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassZoneGraphNavigationUtils.h"
#include "ZoneGraphSubsystem.h"
#include "StateTreeLinker.h"
#include "MassDebugger.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphPathFollowTask)


bool FMassZoneGraphPathFollowTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(LocationHandle);
	Linker.LinkExternalData(MoveTargetHandle);
	Linker.LinkExternalData(PathRequestHandle);
	Linker.LinkExternalData(ShortPathHandle);
	Linker.LinkExternalData(CachedLaneHandle);
	Linker.LinkExternalData(AgentRadiusHandle);
	Linker.LinkExternalData(MovementParamsHandle);
	Linker.LinkExternalData(ZoneGraphSubsystemHandle);

	return true;
}

void FMassZoneGraphPathFollowTask::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly(LocationHandle);
	Builder.AddReadWrite(MoveTargetHandle);
	Builder.AddReadWrite(PathRequestHandle);
	Builder.AddReadWrite(ShortPathHandle);
	Builder.AddReadWrite(CachedLaneHandle);
	Builder.AddReadOnly(AgentRadiusHandle);
	Builder.AddReadOnly(MovementParamsHandle);
	Builder.AddReadOnly(ZoneGraphSubsystemHandle);
}

bool FMassZoneGraphPathFollowTask::RequestPath(FMassStateTreeExecutionContext& Context, const FMassZoneGraphTargetLocation& RequestedTargetLocation) const
{
	const UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetExternalData(ZoneGraphSubsystemHandle);
	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	const FAgentRadiusFragment& AgentRadius = Context.GetExternalData(AgentRadiusHandle);
	FMassZoneGraphShortPathFragment& ShortPath = Context.GetExternalData(ShortPathHandle);
	FMassZoneGraphCachedLaneFragment& CachedLane = Context.GetExternalData(CachedLaneHandle);
	FMassMoveTargetFragment& MoveTarget = Context.GetExternalData(MoveTargetHandle);
	FMassZoneGraphPathRequestFragment& RequestFragment = Context.GetExternalData(PathRequestHandle);
	const FMassMovementParameters& MovementParams = Context.GetExternalData(MovementParamsHandle);

	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Context.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG

	if (RequestedTargetLocation.LaneHandle != LaneLocation.LaneHandle)
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Target location lane %s does not match current lane location %s."),
			*RequestedTargetLocation.LaneHandle.ToString(), *LaneLocation.LaneHandle.ToString());
		return false;
	}

	if (bDisplayDebug)
	{
		MASSBEHAVIOR_LOG(Log, TEXT("PathFollow Request: %s, lane %s, Start: %f End:%f, next lane %s."),
			RequestedTargetLocation.bMoveReverse ? TEXT("reverse") : TEXT("forward"),
			*LaneLocation.LaneHandle.ToString(),
			LaneLocation.DistanceAlongLane, RequestedTargetLocation.TargetDistance,
			*RequestedTargetLocation.NextLaneHandle.ToString());
	}

	// @todo: Combine FMassZoneGraphTargetLocation and FZoneGraphShortPathRequest.
	FZoneGraphShortPathRequest& PathRequest = RequestFragment.PathRequest;
	PathRequest.StartPosition = MoveTarget.Center;
	PathRequest.bMoveReverse = RequestedTargetLocation.bMoveReverse;
	PathRequest.TargetDistance = RequestedTargetLocation.TargetDistance;
	PathRequest.NextLaneHandle = RequestedTargetLocation.NextLaneHandle;
	PathRequest.NextExitLinkType = RequestedTargetLocation.NextExitLinkType;
	PathRequest.EndOfPathIntent = RequestedTargetLocation.EndOfPathIntent;
	PathRequest.bIsEndOfPathPositionSet = RequestedTargetLocation.EndOfPathPosition.IsSet();
	PathRequest.EndOfPathPosition = RequestedTargetLocation.EndOfPathPosition.Get(FVector::ZeroVector);
	PathRequest.bIsEndOfPathDirectionSet = RequestedTargetLocation.EndOfPathDirection.IsSet();
	PathRequest.EndOfPathDirection.Set(RequestedTargetLocation.EndOfPathDirection.Get(FVector::ForwardVector));
	PathRequest.AnticipationDistance = RequestedTargetLocation.AnticipationDistance;
	PathRequest.EndOfPathOffset.Set(FMath::RandRange(-AgentRadius.Radius, AgentRadius.Radius));

	const float DesiredSpeed = FMath::Min(MovementParams.GenerateDesiredSpeed(InstanceData.MovementStyle, Context.GetEntity().Index) * InstanceData.SpeedScale, MovementParams.MaxSpeed);
	const UWorld* World = Context.GetWorld();
	checkf(World != nullptr, TEXT("A valid world is expected from the execution context"));

	MoveTarget.CreateNewAction(EMassMovementAction::Move, *World);
	return UE::MassNavigation::ActivateActionMove(*World, Context.GetOwner(), Context.GetEntity(), ZoneGraphSubsystem, LaneLocation, PathRequest, AgentRadius.Radius, DesiredSpeed, MoveTarget, ShortPath, CachedLane);
}

EStateTreeRunStatus FMassZoneGraphPathFollowTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG
	if (bDisplayDebug)
	{
		MASSBEHAVIOR_LOG(Verbose, TEXT("enterstate."));
	}

	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	const FMassZoneGraphTargetLocation* TargetLocation = InstanceData.TargetLocation.GetMutablePtr<FMassZoneGraphTargetLocation>(Context);

	if (TargetLocation == nullptr)
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Target is not defined."));
		return EStateTreeRunStatus::Failed;
	}

	if (TargetLocation->LaneHandle != LaneLocation.LaneHandle)
	{
		// @todo: Verbosity of this message was downgraded from 'Error' to 'Log' until we have time to investigate.
		// MASSBEHAVIOR_LOG now outputs to the debug log in addition to the visual logger and some sample
		// projects (e.g., CitySample) might spam.
		MASSBEHAVIOR_LOG(Log, TEXT("Target is not on current lane, target lane is %s expected %s."), *TargetLocation->LaneHandle.ToString(), *LaneLocation.LaneHandle.ToString());
		return EStateTreeRunStatus::Failed;
	}

	if (!RequestPath(MassContext, *TargetLocation))
	{
		return EStateTreeRunStatus::Failed;
	}

	return EStateTreeRunStatus::Running;
}

EStateTreeRunStatus FMassZoneGraphPathFollowTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
	const FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
	bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(MassContext.GetEntity());
#endif // WITH_MASSGAMEPLAY_DEBUG
	if (bDisplayDebug)
	{
		MASSBEHAVIOR_LOG(Verbose, TEXT("tick"));
	}

	const FMassZoneGraphShortPathFragment& ShortPath = Context.GetExternalData(ShortPathHandle);
	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	const FMassZoneGraphTargetLocation* TargetLocation = InstanceData.TargetLocation.GetMutablePtr<FMassZoneGraphTargetLocation>(Context);

	// Current path follow is done, but it was partial (i.e. many points on a curve), try again until we get there.
	if (ShortPath.IsDone() && ShortPath.bPartialResult)
	{
		if (TargetLocation == nullptr)
		{
			MASSBEHAVIOR_LOG(Error, TEXT("Target is not defined."));
			return EStateTreeRunStatus::Failed;
		}
		if (TargetLocation->LaneHandle != LaneLocation.LaneHandle)
		{
			MASSBEHAVIOR_LOG(Error, TEXT("Target is not on current lane, target lane is %s expected %s."), *TargetLocation->LaneHandle.ToString(), *LaneLocation.LaneHandle.ToString());
			return EStateTreeRunStatus::Failed;
		}

		if (!RequestPath(MassContext, *TargetLocation))
		{
			MASSBEHAVIOR_LOG(Error, TEXT("Failed to request path."));
			return EStateTreeRunStatus::Failed;
		}
	}

	return ShortPath.IsDone() ? EStateTreeRunStatus::Succeeded : EStateTreeRunStatus::Running;
}
