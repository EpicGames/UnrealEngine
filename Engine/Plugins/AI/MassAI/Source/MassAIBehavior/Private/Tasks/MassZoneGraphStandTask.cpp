// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tasks/MassZoneGraphStandTask.h"
#include "StateTreeExecutionContext.h"
#include "ZoneGraphSubsystem.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassZoneGraphNavigationUtils.h"
#include "MassAIBehaviorTypes.h"
#include "MassNavigationFragments.h"
#include "MassMovementFragments.h"
#include "MassStateTreeExecutionContext.h"
#include "MassSignalSubsystem.h"
#include "MassSimulationLOD.h"
#include "StateTreeLinker.h"
#include "MassStateTreeDependency.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphStandTask)

bool FMassZoneGraphStandTask::Link(FStateTreeLinker& Linker)
{
	Linker.LinkExternalData(LocationHandle);
	Linker.LinkExternalData(MoveTargetHandle);
	Linker.LinkExternalData(ShortPathHandle);
	Linker.LinkExternalData(CachedLaneHandle);
	Linker.LinkExternalData(MovementParamsHandle);
	Linker.LinkExternalData(ZoneGraphSubsystemHandle);
	Linker.LinkExternalData(MassSignalSubsystemHandle);

	return true;
}

void FMassZoneGraphStandTask::GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
{
	Builder.AddReadOnly(LocationHandle);
	Builder.AddReadWrite(MoveTargetHandle);
	Builder.AddReadWrite(ShortPathHandle);
	Builder.AddReadWrite(CachedLaneHandle);
	Builder.AddReadOnly(MovementParamsHandle);
	Builder.AddReadOnly(ZoneGraphSubsystemHandle);
	Builder.AddReadWrite(MassSignalSubsystemHandle);
}

EStateTreeRunStatus FMassZoneGraphStandTask::EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const
{
	const FMassStateTreeExecutionContext& MassStateTreeContext = static_cast<FMassStateTreeExecutionContext&>(Context);

	const FMassZoneGraphLaneLocationFragment& LaneLocation = Context.GetExternalData(LocationHandle);
	const UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetExternalData(ZoneGraphSubsystemHandle);
	const FMassMovementParameters& MovementParams = Context.GetExternalData(MovementParamsHandle);

	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);

	if (!LaneLocation.LaneHandle.IsValid())
	{
		MASSBEHAVIOR_LOG(Error, TEXT("Invalid lande handle"));
		return EStateTreeRunStatus::Failed;
	}

	FMassZoneGraphShortPathFragment& ShortPath = Context.GetExternalData(ShortPathHandle);
	FMassZoneGraphCachedLaneFragment& CachedLane = Context.GetExternalData(CachedLaneHandle);
	FMassMoveTargetFragment& MoveTarget = Context.GetExternalData(MoveTargetHandle);

	// TODO: This could be smarter too, like having a stand location/direction, or even make a small path to stop, if we're currently running.

	const UWorld* World = Context.GetWorld();
	checkf(World != nullptr, TEXT("A valid world is expected from the execution context"));

	MoveTarget.CreateNewAction(EMassMovementAction::Stand, *World);
	const bool bSuccess = UE::MassNavigation::ActivateActionStand(*World, Context.GetOwner(), MassStateTreeContext.GetEntity(), ZoneGraphSubsystem, LaneLocation, MovementParams.DefaultDesiredSpeed, MoveTarget, ShortPath, CachedLane);
	if (!bSuccess)
	{
		return EStateTreeRunStatus::Failed;
	}

	InstanceData.Time = 0.0f;

	// A Duration <= 0 indicates that the task runs until a transition in the state tree stops it.
	// Otherwise we schedule a signal to end the task.
	if (InstanceData.Duration > 0.0f)
	{
		UMassSignalSubsystem& MassSignalSubsystem = Context.GetExternalData(MassSignalSubsystemHandle);
		MassSignalSubsystem.DelaySignalEntityDeferred(MassStateTreeContext.GetMassEntityExecutionContext(), UE::Mass::Signals::StandTaskFinished, MassStateTreeContext.GetEntity(), InstanceData.Duration);
	}

	return EStateTreeRunStatus::Running;
}

EStateTreeRunStatus FMassZoneGraphStandTask::Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const
{
	FInstanceDataType& InstanceData = Context.GetInstanceData(*this);
	
	InstanceData.Time += DeltaTime;
	return InstanceData.Duration <= 0.0f ? EStateTreeRunStatus::Running : (InstanceData.Time < InstanceData.Duration ? EStateTreeRunStatus::Running : EStateTreeRunStatus::Succeeded);
}
