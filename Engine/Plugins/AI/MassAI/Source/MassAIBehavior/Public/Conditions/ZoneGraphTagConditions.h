// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "MassStateTreeTypes.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphTagConditions.generated.h"

#define UE_API MASSAIBEHAVIOR_API

/**
* ZoneGraph Tag condition.
*/

USTRUCT()
struct FZoneGraphTagFilterConditionInstanceData
{
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Input)
	FZoneGraphTagMask Tags = FZoneGraphTagMask::None;
};

USTRUCT(DisplayName="ZoneGraphTagFilter Compare")
struct FZoneGraphTagFilterCondition : public FMassStateTreeConditionBase
{
	GENERATED_BODY()

	using FInstanceDataType = FZoneGraphTagFilterConditionInstanceData;

	FZoneGraphTagFilterCondition() = default;
	
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = Condition)
	FZoneGraphTagFilter Filter;

	UPROPERTY(EditAnywhere, Category = Condition)
	bool bInvert = false;
};

/**
* ZoneGraph Tag condition.
*/

USTRUCT()
struct FZoneGraphTagMaskConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Input)
	FZoneGraphTagMask Left = FZoneGraphTagMask::None;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FZoneGraphTagMask Right = FZoneGraphTagMask::None;
};

USTRUCT(DisplayName="ZoneGraphTagMask Compare")
struct FZoneGraphTagMaskCondition : public FMassStateTreeConditionBase
{
	GENERATED_BODY()

	using FInstanceDataType = FZoneGraphTagMaskConditionInstanceData;

	FZoneGraphTagMaskCondition() = default;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = Condition)
	EZoneLaneTagMaskComparison Operator = EZoneLaneTagMaskComparison::Any;

	UPROPERTY(EditAnywhere, Category = Condition)
	bool bInvert = false;
};

/**
* ZoneGraph Tag condition.
*/

USTRUCT()
struct FZoneGraphTagConditionInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Input)
	FZoneGraphTag Left = FZoneGraphTag::None;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FZoneGraphTag Right = FZoneGraphTag::None;
};

USTRUCT(DisplayName="ZoneGraphTag Compare")
struct FZoneGraphTagCondition : public FMassStateTreeConditionBase
{
	GENERATED_BODY()

	using FInstanceDataType = FZoneGraphTagConditionInstanceData;

	FZoneGraphTagCondition() = default;

	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual bool TestCondition(FStateTreeExecutionContext& Context) const override;

	UPROPERTY(EditAnywhere, Category = Condition)
	bool bInvert = false;
};

#undef UE_API
