// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassComponentHitSubsystem.h"
#include "MassStateTreeTypes.h"
#include "MassComponentHitEvaluator.generated.h"

#define UE_API MASSAIBEHAVIOR_API

class UMassComponentHitSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
}

/**
 * Evaluator to extract last hit from the MassComponentHitSubsystem and expose it for tasks and transitions
 */

USTRUCT()
struct FMassComponentHitEvaluatorInstanceData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Output)
	bool bGotHit = false;

	UPROPERTY(VisibleAnywhere, Category = Output)
	FMassEntityHandle LastHitEntity;
};

USTRUCT(meta = (DisplayName = "Mass ComponentHit Eval"))
struct FMassComponentHitEvaluator : public FMassStateTreeEvaluatorBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassComponentHitEvaluatorInstanceData;
	
protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual void Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<UMassComponentHitSubsystem> ComponentHitSubsystemHandle;
};

#undef UE_API
