// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "ZoneGraphTypes.h"
#include "MassZoneGraphAnnotationEvaluator.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FStateTreeExecutionContext;
struct FMassZoneGraphAnnotationFragment;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
 * Evaluator to expose ZoneGraph Annotation Tags for decision-making.
 */
USTRUCT()
struct FMassZoneGraphAnnotationEvaluatorInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Output)
	FZoneGraphTagMask AnnotationTags = FZoneGraphTagMask::None;
};

USTRUCT(meta = (DisplayName = "ZG Annotation Tags"))
struct FMassZoneGraphAnnotationEvaluator : public FMassStateTreeEvaluatorBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassZoneGraphAnnotationEvaluatorInstanceData;

	UE_API FMassZoneGraphAnnotationEvaluator();

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual void Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<FMassZoneGraphAnnotationFragment> AnnotationTagsFragmentHandle;
};

#undef UE_API
