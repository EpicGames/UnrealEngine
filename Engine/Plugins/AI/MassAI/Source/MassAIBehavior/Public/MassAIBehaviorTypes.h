// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Logging/LogMacros.h"
#include "MassCommonTypes.h"
#include "VisualLogger/VisualLogger.h"

MASSAIBEHAVIOR_API DECLARE_LOG_CATEGORY_EXTERN(LogMassBehavior, Warning, All);

namespace UE::Mass::ProcessorGroupNames
{
	const FName UpdateAnnotationTags = FName(TEXT("UpdateAnnotationTags"));
}

/**
 * Helper macros that could be used inside FMassStateTreeEvaluators and FMassStateTreeTasks.
 * Requirements is a property or parameters with the following declaration: FStateTreeExecutionContext& Context
 * These macros should be used to standardize the output format and simplify code at call site.
 * They could also easily be changed from UE_(C)VLOG to UE_(C)VLOG_UELOG (or any other implementation) in one go.
 *  e.g. the following:
 *		#if WITH_MASSGAMEPLAY_DEBUG
 *			const FMassStateTreeExecutionContext& MassContext = static_cast<FMassStateTreeExecutionContext&>(Context);
 *			UE_VLOG(MassContext.GetOwner(), LogMassBehavior, Log, TEXT("Entity [%s]: Starting action: %s"), *MassContext.GetEntity().DebugGetDescription(), *StaticEnum<ESomeActionEnum>()->GetValueAsString(SomeActionEnumValue));
 *		#endif // WITH_MASSGAMEPLAY_DEBUG
 *
 *	could be replaced by:
 *		MASSBEHAVIOR_CLOG(bDisplayDebug, Log, TEXT("Starting action: %s"), *StaticEnum<ESomeActionEnum>()->GetValueAsString(SomeActionEnumValue));
 */
#if WITH_MASSGAMEPLAY_DEBUG
#define MASSBEHAVIOR_LOG(Verbosity, Format, ...) UE_VLOG_UELOG(static_cast<const FMassStateTreeExecutionContext&>(Context).GetOwner(), LogMassBehavior, Verbosity, \
	TEXT("Entity [%s][%s] ") Format, *static_cast<const FMassStateTreeExecutionContext&>(Context).GetEntity().DebugGetDescription(), *StaticStruct()->GetName(), ##__VA_ARGS__)
#define MASSBEHAVIOR_CLOG(Condition, Verbosity, Format, ...) UE_CVLOG_UELOG((Condition), static_cast<const FMassStateTreeExecutionContext&>(Context).GetOwner(), LogMassBehavior, Verbosity, \
	TEXT("Entity [%s][%s] ") Format, *static_cast<const FMassStateTreeExecutionContext&>(Context).GetEntity().DebugGetDescription(), *StaticStruct()->GetName(), ##__VA_ARGS__)
#else
#define MASSBEHAVIOR_LOG(Verbosity, Format, ...)
#define MASSBEHAVIOR_CLOG(Condition, Verbosity, Format, ...)
#endif // WITH_MASSGAMEPLAY_DEBUG
