// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSettings.h"
#include "MassLODTypes.h"
#include "MassBehaviorSettings.generated.h"

#define UE_API MASSAIBEHAVIOR_API

class UMassStateTreeProcessor;

UCLASS(MinimalAPI, config = Mass, defaultconfig, meta = (DisplayName = "Mass Behavior"))
class UMassBehaviorSettings : public UMassModuleSettings
{
	GENERATED_BODY()

public:
	UE_API UMassBehaviorSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());
	
	UPROPERTY(EditAnywhere, Category = "Mass|LOD", config)
	int32 MaxActivationsPerLOD[EMassLOD::Max];

	/**
	 * Class to use when creating dynamic processors to handle given StateTree assets.
	 * Can be also set via DefaultMass.ini file. 
	 */
	UPROPERTY(Config, EditAnywhere, Category="Mass|StateTree", NoClear)
	TSoftClassPtr<UMassStateTreeProcessor> DynamicStateTreeProcessorClass;
};

#undef UE_API
