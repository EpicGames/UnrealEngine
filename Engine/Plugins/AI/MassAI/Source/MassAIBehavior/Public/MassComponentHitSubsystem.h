// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassComponentHitTypes.h"
#include "MassSubsystemBase.h"
#include "MassComponentHitSubsystem.generated.h"

#define UE_API MASSAIBEHAVIOR_API

class UMassAgentSubsystem;
class UMassSignalSubsystem;
class UCapsuleComponent;


/**
 * Subsystem that keeps track of the latest component hits and allow mass entities to retrieve and handle them
 */
UCLASS(MinimalAPI)
class UMassComponentHitSubsystem : public UMassTickableSubsystemBase
{
	GENERATED_BODY()

public:
	UE_API const FMassHitResult* GetLastHit(const FMassEntityHandle Entity) const;

protected:
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual void Deinitialize() override;
	UE_API virtual void Tick(float DeltaTime) override;
	UE_API virtual TStatId GetStatId() const override;
	
	UE_API void RegisterForComponentHit(const FMassEntityHandle Entity, UCapsuleComponent& CapsuleComponent);
	UE_API void UnregisterForComponentHit(FMassEntityHandle Entity, UCapsuleComponent& CapsuleComponent);

	UFUNCTION()
	UE_API void OnHitCallback(UPrimitiveComponent* HitComp, AActor* OtherActor, UPrimitiveComponent* OtherComp, FVector NormalImpulse, const FHitResult& Hit);

	UPROPERTY()
	TObjectPtr<UMassSignalSubsystem> SignalSubsystem;

	UPROPERTY()
    TObjectPtr<UMassAgentSubsystem> AgentSubsystem;

	UPROPERTY()
	TMap<FMassEntityHandle, FMassHitResult> HitResults;

	UPROPERTY()
	TMap<TObjectPtr<UActorComponent>, FMassEntityHandle> ComponentToEntityMap;

	UPROPERTY()
	TMap<FMassEntityHandle, TObjectPtr<UActorComponent>> EntityToComponentMap;
};

template<>
struct TMassExternalSubsystemTraits<UMassComponentHitSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false,
	};
};

#undef UE_API
