// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/EngineTypes.h"
#include "Mass/EntityHandle.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "StateTreeTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassComponentHitTypes.generated.h"

namespace UE::Mass::Signals
{
const FName HitReceived = FName(TEXT("HitReceived"));
}

USTRUCT()
struct FMassHitResult
{
	GENERATED_BODY()

	FMassHitResult() = default;

	FMassHitResult(const FMassEntityHandle OtherEntity, const double Time)
		: OtherEntity(OtherEntity)
		, HitTime(Time)
		, LastFilteredHitTime(Time)
	{
	}

	bool IsValid() const { return OtherEntity.IsValid(); }
	
	FMassEntityHandle OtherEntity;

	/** Time when first hit was received. */
	double HitTime = 0.;

	/** Time used for filtering frequent hits. */
	double LastFilteredHitTime = 0.;
};
