// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#include "MassEntityTypes.h"
#include "MassCommonTypes.h"
#include "MassLookAtTypes.h"
#include "ZoneGraphTypes.h"
#include "Containers/StaticArray.h"
#include "MassLookAtFragments.generated.h"

#define UE_API MASSAIBEHAVIOR_API

/** Primary look at mode, gazing can be applied on top. */
UENUM()
enum class EMassLookAtMode : uint8
{
	/** Look forward */
	LookForward,
	/** Look along the current path */
	LookAlongPath,
	/** Track specified entity */
	LookAtEntity,
};

/** Different gaze modes applied on top of the look at mode. */
UENUM()
enum class EMassLookAtGazeMode : uint8
{
	/** No gazing */
	None,
	/** Look constantly in gaze direction until next gaze target is picked. */
	Constant,
	/** Quick look at gaze target, ease in back to main look direction. */
	Glance,
};

/**
 * Struct that holds all parameters of the current entity look at 
 */
USTRUCT()
struct FMassLookAtFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Resets gaze related properties in the fragment to their default values */
	void ResetGaze()
	{
		GazeTargetLocation = FVector::ZeroVector;
		GazeDirection = FVector::ForwardVector;
		GazeTrackedEntity.Reset();
		GazeStartTime = 0.0f;
		GazeDuration = 0.0f;
		RandomGazeMode = EMassLookAtGazeMode::None;
		RandomGazeYawVariation = 0;
		RandomGazePitchVariation = 0;
		bRandomGazeEntities = false;
	}

	/** Resets all properties in the fragment to their default values */
	void ResetAll()
	{
		ResetMainLookAt();
		ResetGaze();
	}

	/** Resets main LookAt related properties in the fragment to their default values */
	void ResetMainLookAt()
	{
		MainTargetLocation = FVector::ZeroVector;
		Direction = FVector::ForwardVector;
		TrackedEntity.Reset();
		LastSeenActionID = 0;
		LookAtMode = EMassLookAtMode::LookForward;
	}

	/** Resets gaze and main LookAt (if override not active) related properties in the fragment to their default values */
	void ResetSystemicLookAt()
	{
		if (OverrideState == EOverrideState::ActiveSystemicOnly
			|| OverrideState == EOverrideState::OverridenSystemic
			|| OverrideState == EOverrideState::PendingSystemicReactivation)
		{
			ResetGaze();

			// Only reset main LookAt properties if no override is active.
			// In that case it will be handled by the LookAtSubsystem when updating the override requests.
			if (OverrideState == EOverrideState::ActiveSystemicOnly
				|| OverrideState == EOverrideState::PendingSystemicReactivation)
			{
				ResetMainLookAt();
				OverrideState = EOverrideState::AllDisabled;
			}
			else
			{
				OverrideState = EOverrideState::ActiveOverrideOnly;
			}
		}
	}

	/** Current look at main target location. */
	UPROPERTY(Transient)
	FVector MainTargetLocation = FVector::ZeroVector;

	/** Current gaze target location. */
	UPROPERTY(Transient)
	FVector GazeTargetLocation = FVector::ZeroVector;

	/** Current look at direction (with gaze applied). */
	UPROPERTY(Transient)
	FVector Direction = FVector::ForwardVector;

	/** Current gaze direction, applied on top of look at direction based on gaze mode. */
	UPROPERTY(Transient)
	FVector GazeDirection = FVector::ForwardVector;

	/** Specific entity that is being tracked as primary look at. */
	UPROPERTY(Transient)
	FMassEntityHandle TrackedEntity;

	/** Entity that is tracked as part of gazing. */
	UPROPERTY(Transient)
	FMassEntityHandle GazeTrackedEntity;

	/** Start time of the current gaze. */
	UPROPERTY(Transient)
	double GazeStartTime = 0.;

	/** Duration of the current gaze. */
	UPROPERTY(Transient)
	float GazeDuration = 0.0f;

	/** Last seen action ID, used to check when look at trajectory needs to be updated. */
	UPROPERTY(Transient)
	uint16 LastSeenActionID = 0;

	/** Primary look at mode. */
	UPROPERTY(Transient)
	EMassLookAtMode LookAtMode = EMassLookAtMode::LookForward;

	/** Primary look at interpolation speed (not used by the LookAt processor but can be forwarded to the animation system). */
	UPROPERTY(Transient)
	EMassLookAtInterpolationSpeed InterpolationSpeed = EMassLookAtInterpolationSpeed::Regular;

	/**
	 * Primary look at custom interpolation speed used when 'InterpolationSpeed = EMassLookAtInterpolationSpeed::Custom'
	 * (not used by the LookAt processor but can be forwarded to the animation system).
	 */
	UPROPERTY(Transient)
	float CustomInterpolationSpeed = UE::Mass::LookAt::DefaultCustomInterpolationSpeed;

	/** Gaze look at mode. */
	UPROPERTY(Transient)
	EMassLookAtGazeMode RandomGazeMode = EMassLookAtGazeMode::None;

	/** Random gaze angle yaw variation (in degrees). */
	UPROPERTY(Transient)
	uint8 RandomGazeYawVariation = 0;

	/** Random gaze angle pitch variation (in degrees). */
	UPROPERTY(Transient)
	uint8 RandomGazePitchVariation = 0;

	/** Whether random gaze can also pick interesting entities to look at. */
	UPROPERTY(Transient)
	uint8 bRandomGazeEntities : 1 = false;

	enum class EOverrideState : uint8
	{
		/** No active task and no active override */
		AllDisabled,

		/** No active task, only an active override */
		ActiveOverrideOnly,

		/** Active task only, no active override */
		ActiveSystemicOnly,

		/** Active task overriden */
		OverridenSystemic,

		/**
		 * Intermediate state used from 'OverridenSystemic' after removing last override request
		 * to allow active task to reapply its values and transition to 'ActiveSystemicOnly'
		 */
		PendingSystemicReactivation
	};

	/** Current state based on the systemic task and active overrides */
	EOverrideState OverrideState : 7 = EOverrideState::AllDisabled;
};

/**
 * Special tag to mark an entity that could be tracked by the LookAt
 */
USTRUCT()
struct UE_DEPRECATED(5.6, "Use FMassLookAtTargetFragment instead") MASSAIBEHAVIOR_API FMassLookAtTargetTag : public FMassTag
{
	GENERATED_BODY()
};

/**
 * Struct that holds all parameters of a look at request
 */
USTRUCT()
struct FMassLookAtRequestFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassLookAtRequestFragment() = default;
	FMassLookAtRequestFragment(const FMassEntityHandle ViewerEntity, const FMassLookAtPriority Priority, const EMassLookAtMode Mode)
		: ViewerEntity(ViewerEntity)
		, Priority(Priority)
		, LookAtMode(Mode)
	{
	}

	FMassLookAtRequestFragment(
		const FMassEntityHandle ViewerEntity
		, const FMassLookAtPriority Priority
		, const EMassLookAtMode Mode
		, const FMassEntityHandle Target
		, const EMassLookAtInterpolationSpeed InterpolationSpeed
		, const float CustomInterpolationSpeed)
		: FMassLookAtRequestFragment(ViewerEntity, Priority, Mode)
	{
		TargetEntity = Target;
		this->InterpolationSpeed = InterpolationSpeed;
		this->CustomInterpolationSpeed = CustomInterpolationSpeed;
	}

	UPROPERTY(Transient)
	FMassEntityHandle ViewerEntity;

	UPROPERTY(Transient)
	FMassEntityHandle TargetEntity;

	UPROPERTY(Transient)
	FMassLookAtPriority Priority{static_cast<uint8>(EMassLookAtPriorities::LowestPriority)};

	UPROPERTY(Transient)
	EMassLookAtMode LookAtMode = EMassLookAtMode::LookForward;

	UPROPERTY(Transient)
	EMassLookAtInterpolationSpeed InterpolationSpeed = EMassLookAtInterpolationSpeed::Regular;

	UPROPERTY(Transient)
	float CustomInterpolationSpeed = UE::Mass::LookAt::DefaultCustomInterpolationSpeed;

	friend FString LexToString(const FMassLookAtRequestFragment& Fragment)
	{
		if (Fragment.LookAtMode == EMassLookAtMode::LookAtEntity)
		{
			return FString::Printf(TEXT("Priority = %d Mode = %s Target = [%s]")
			, Fragment.Priority.Get()
			, *UEnum::GetValueAsString(Fragment.LookAtMode)
			, *LexToString(Fragment.TargetEntity));
		}

		return FString::Printf(TEXT("Priority=%d Mode=%s")
			, Fragment.Priority.Get()
			, *UEnum::GetValueAsString(Fragment.LookAtMode));
	}
};

/**
 * Fragment to mark an entity that could be tracked by the LookAt processor
 * and providing information that could be used to get more accurate locations.
 */
USTRUCT()
struct FMassLookAtTargetFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Offset in local space to add to the target transform to get the final location */
	UPROPERTY(Transient)
	FVector Offset = FVector::ZeroVector;

	/** When a viewer is searching for a random target this priority will influence the selected target */
	UPROPERTY(Transient)
	FMassLookAtPriority Priority{static_cast<uint8>(EMassLookAtPriorities::LowestPriority)};

	UE::Mass::LookAt::FTargetHashGrid2D::FCellLocation CellLocation;
};

/** Tag to tell if the entity is in the LookAt target grid */
USTRUCT()
struct FMassInLookAtTargetGridTag : public FMassTag
{
	GENERATED_BODY()
};

USTRUCT()
struct FMassLookAtTrajectoryPoint
{
	GENERATED_BODY()

	void Set(const FVector InPosition, const FVector2D InTangent, const float InDistanceAlongLane)
	{
		Position = InPosition;
		Tangent.Set(InTangent);
		DistanceAlongLane.Set(InDistanceAlongLane);
	}
	
	/** Position of the path. */
	FVector Position = FVector::ZeroVector;
	
	/** Tangent direction of the path. */
	FMassSnorm8Vector2D Tangent;
	
	/** Position of the point along the original path. (Could potentially be uint16 at 10cm accuracy) */
	FMassInt16Real10 DistanceAlongLane = FMassInt16Real10(0.0f);
};

USTRUCT()
struct FMassLookAtTrajectoryFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassLookAtTrajectoryFragment() = default;
	
	static constexpr uint8 MaxPoints = 3;

	void Reset()
	{
		NumPoints = 0;
	}

	bool AddPoint(const FVector Position, const FVector2D Tangent, const float DistanceAlongLane)
	{
		if (NumPoints < MaxPoints)
		{
			FMassLookAtTrajectoryPoint& Point = Points[NumPoints++];
			Point.Set(Position, Tangent, DistanceAlongLane);
			return true;
		}
		return false;
	}

	UE_API FVector GetPointAtDistanceExtrapolated(const float DistanceAlongPath) const;
	
	/** Path points */
	TStaticArray<FMassLookAtTrajectoryPoint, MaxPoints> Points;

	/** Lane handle the trajectory was build for. */
	FZoneGraphLaneHandle LaneHandle;

	/** Number of points on path. */
	uint8 NumPoints = 0;

	bool bMoveReverse = false;
};

#undef UE_API
