// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassObserverProcessor.h"
#include "MassProcessor.h"
#include "MassEntityQuery.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassSignalProcessorBase.h"
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassLookAtProcessors.generated.h"

class UMassLookAtSubsystem;
class UMassNavigationSubsystem;
class UZoneGraphSubsystem;
struct FMassLookAtFragment;
struct FMassZoneGraphLaneLocationFragment;
struct FMassLookAtTrajectoryFragment;
struct FMassZoneGraphShortPathFragment;
struct FMassMoveTargetFragment;

/**
 * Processor to choose and assign LookAt configurations  
 */
UCLASS(MinimalAPI)
class UMassLookAtProcessor : public UMassProcessor
{
	GENERATED_BODY()

	UMassLookAtProcessor();

protected:

	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	/** Selects a nearby target if possible or use a random fixed direction */
	MASSAIBEHAVIOR_API void FindNewGazeTarget(const UMassNavigationSubsystem& MassNavSystem
		, const UMassLookAtSubsystem& LookAtSystem
		, const FMassEntityManager& EntityManager
		, const double CurrentTime
		, const FTransform& Transform
		, FMassEntityHandle Entity
		, FMassLookAtFragment& LookAt) const;

	/** Updates look direction based on look at trajectory. */
	MASSAIBEHAVIOR_API void UpdateLookAtTrajectory(const FTransform& Transform, const FMassZoneGraphLaneLocationFragment& ZoneGraphLocation,
								const FMassLookAtTrajectoryFragment& LookAtTrajectory, const bool bDisplayDebug, FMassLookAtFragment& LookAt) const;

	/** Updates look at based on tracked entity. */
	MASSAIBEHAVIOR_API void UpdateLookAtTrackedEntity(const FMassEntityManager& EntityManager, const FTransform& Transform, const bool bDisplayDebug, FMassLookAtFragment& LookAt) const;

	/** Updates gaze based on tracked entity. */
	MASSAIBEHAVIOR_API bool UpdateGazeTrackedEntity(const FMassEntityManager& EntityManager, const FTransform& Transform, const bool bDisplayDebug, FMassLookAtFragment& LookAt) const;

	/** Builds look at trajectory along the current path. */
	MASSAIBEHAVIOR_API void BuildTrajectory(const UZoneGraphSubsystem& ZoneGraphSubsystem, const FMassZoneGraphLaneLocationFragment& LaneLocation, const FMassZoneGraphShortPathFragment& ShortPath,
							const FMassEntityHandle Entity, const bool bDisplayDebug, FMassLookAtTrajectoryFragment& LookAtTrajectory);

	/** Size of the query to find potential targets */
	UPROPERTY(EditDefaultsOnly, Category = LookAt, config, meta = (UIMin = 0.0, ClampMin = 0.0))
	float QueryExtent = 0.f;

	/** Time an entity must use a random look at. */
	UPROPERTY(EditDefaultsOnly, Category = LookAt, config, meta = (UIMin = 0.0, ClampMin = 0.0))
	float Duration = 0.f;
	
	/** Variation applied to a random look at duration [Duration-Variation : Duration+Variation] */
	UPROPERTY(EditDefaultsOnly, Category = LookAt, config, meta = (UIMin = 0.0, ClampMin = 0.0))
	float DurationVariation = 0.f;

	/** Height offset that will be added for debug draw of the look at vector. Useful to bring arrow near character's eyes */
	UPROPERTY(EditDefaultsOnly, Category = LookAt, config, meta = (UIMin = 0.0, ClampMin = 0.0, DisplayName="Debug draw Z offset (cm)"))
	float DebugZOffset = 0.f;

	/** Tolerance in degrees between the forward direction and the look at duration to track an entity */
	UPROPERTY(EditDefaultsOnly, Category = LookAt, config, meta = (UIMin = 0.0, ClampMin = 0.0, UIMax = 180.0, ClampMax = 180.0, DisplayName="Angle Threshold (degrees)"))
	float AngleThresholdInDegrees = 0.f;

	FMassEntityQuery EntityQuery_Conditional;
};

/**
 * Processor to maintain a list of LookAt targets in a spatial query structure in the subsystem
 */
UCLASS(MinimalAPI)
class UMassLookAtTargetGridProcessor : public UMassProcessor
{
	GENERATED_BODY()

	UMassLookAtTargetGridProcessor();

protected:

	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery AddToGridQuery;
	FMassEntityQuery UpdateGridQuery;
	FMassEntityQuery RemoveFromGridQuery;
};

/** Deinitializer processor to remove targets from the hash grid */
UCLASS(MinimalAPI)
class UMassLookAtTargetRemoverProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()

	UMassLookAtTargetRemoverProcessor();

protected:
	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery Query;
};

/** Initializer processing new LookAt requests */
UCLASS(MinimalAPI)
class UMassLookAtRequestInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

	UMassLookAtRequestInitializer();

protected:
	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery Query;
};


/** Deinitializer processing deleted LookAt requests */
UCLASS(MinimalAPI)
class UMassLookAtRequestDeinitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

	UMassLookAtRequestDeinitializer();

protected:
	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery Query;
};
