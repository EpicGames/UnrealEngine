// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Engine/DeveloperSettings.h"
#include "MassLookAtTypes.h"
#include "MassLookAtSettings.generated.h"

DECLARE_MULTICAST_DELEGATE(FOnMassLookAtPrioritiesChanged);

/**
 * Implements the settings for the MassLookAt module.
 */
UCLASS(MinimalAPI, config = Plugins, defaultconfig, DisplayName = "Mass LookAt")
class UMassLookAtSettings : public UDeveloperSettings
{
	GENERATED_BODY()
public:
	explicit UMassLookAtSettings(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	virtual void PostEditChangeChainProperty(FPropertyChangedChainEvent& PropertyChangedEvent) override;
#endif

	MASSAIBEHAVIOR_API void GetValidPriorityInfos(TArray<FMassLookAtPriorityInfo>& OutInfos) const;

	/** @return Additional offset to add to the target location (aka eyes height) */
	[[nodiscard]] FVector GetDefaultTargetLocationOffset() const
	{
		return DefaultTargetLocationOffset;
	}

	/**
	 * Specifies a modifier applied to the target height, if available, to adjust the final location to look at.
	 * Value of 0 represents the base location and a value of 1 the base location with the full target height offset.
	 * TargetHeightRatio is applied before FixedOffsetFromTargetHeight* 
	 * @return Modifier applied on the target height to compute the final location to look at
	 */
	[[nodiscard]] double GetTargetHeightRatio() const
	{
		return TargetHeightRatio;
	}

	/**
	 * Specifies the offset to add to the target height, if available, to adjust the final location to look at.
	 * Negative value should be used to lower the target.
	 * @return Offset applied on the target height to compute the final location to look at
	 */
	[[nodiscard]] double GetFixedOffsetFromTargetHeight() const
	{
		return FixedOffsetFromTargetHeight;
	}

	static MASSAIBEHAVIOR_API FOnMassLookAtPrioritiesChanged OnMassLookAtPrioritiesChanged;

protected:

	UPROPERTY(EditAnywhere, config, Category = LookAt)
	FMassLookAtPriorityInfo Priorities[static_cast<int32>(EMassLookAtPriorities::MaxPriorities)];

	/**
	 * Additional offset added to the target base location (i.e., TransformFragment).
	 * This is used by default when no specific height information (i.e., LookAtTargetTrait initializer) is available.
	 */
	UPROPERTY(EditAnywhere, config, Category = LookAt)
	FVector DefaultTargetLocationOffset = FVector::ZeroVector;

	/**
	 * Optional height modifier ratio applied to the target height, if available, to adjust the final location to look at.
	 * Value of 0 represents the base location and a value of 1 the base location with the full target height offset.
	 * TargetHeightRatio is applied before FixedOffsetFromTargetHeight
	 * @see FixedOffsetFromTargetHeight
	 */
	UPROPERTY(EditAnywhere, config, Category = LookAt, meta = (EditCondition = bShouldUseCapsuleComponentToSetTargetOffset))
	double TargetHeightRatio = 1.0;

	/**
	 * Optional fixed offset (in cm) added to the target height, if available, to adjust the final location to look at.
	 * Negative value should be used to lower the target.
	 * TargetHeightRatio is applied before FixedOffsetFromTargetHeight
	 * @see TargetHeightRatio
	 */
	UPROPERTY(EditAnywhere, config, Category = LookAt, meta = (EditCondition = bShouldUseCapsuleComponentToSetTargetOffset))
	double FixedOffsetFromTargetHeight = 0;
};