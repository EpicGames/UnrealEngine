// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassExecutionContext.h"
#include "MassLookAtFragments.h"
#include "MassSubsystemBase.h"
#include "MassLookAtTypes.h"
#include "MassLookAtSubsystem.generated.h"

#define UE_API MASSAIBEHAVIOR_API

USTRUCT(BlueprintType)
struct FMassLookAtRequestHandle
{
	GENERATED_BODY()

	FMassLookAtRequestHandle() = default;

	FMassLookAtRequestHandle(const FMassEntityHandle& Request, const FMassEntityHandle& Target, const bool bTargetEntityOwnedByRequest)
		: Request(Request)
		, Target(Target)
		, bTargetEntityOwnedByRequest(bTargetEntityOwnedByRequest)
	{
	}

	UPROPERTY()
	FMassEntityHandle Request;

	UPROPERTY()
	FMassEntityHandle Target;

	/** Indicates if the target entity was created for the request and must be destroyed when deleting the request (e.g. CreateLookAtPositionRequest) */
	UPROPERTY()
	bool bTargetEntityOwnedByRequest = false;
};

/**
 * Subsystem that keeps track of the LookAt targets
 */
UCLASS(MinimalAPI)
class UMassLookAtSubsystem : public UMassTickableSubsystemBase
{
	GENERATED_BODY()

public:

	/**
	 * Struct representing a request for a given entity to perform a LookAt action
	 */
	struct FRequest
	{
		FMassEntityHandle RequestHandle;
		FMassLookAtRequestFragment Parameters;
		bool bActive = false;
	};

	/**
	 * Creates a new LookAt request using the provided parameters for the mass entity associated to 'ViewerActor', if any.
	 * @param ViewerActor Actor associated to the mass entity that needs to perform the LookAt
	 * @param Priority Priority assigned to the request that will influence selected request when multiple requests are sent to the same entity
	 * @param TargetLocation Static location used as target
	 * @param InterpolationSpeed Optional parameter to use predefined interpolation speed
	 * @param CustomInterpolationSpeed Optional parameter to specify a custom interpolation speed when 'InterpolationSpeed' is set to EMassLookAtInterpolationSpeed::Custom
	 * Method will log an error if it fails (e.g., no associated mass entity).
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "LookAt", meta = (ReturnDisplayName="Request Handle"))
	UE_API FMassLookAtRequestHandle CreateLookAtPositionRequest(AActor* ViewerActor
		, FMassLookAtPriority Priority
		, FVector TargetLocation
		, EMassLookAtInterpolationSpeed InterpolationSpeed = EMassLookAtInterpolationSpeed::Regular
		, float CustomInterpolationSpeed = 1.5f) const;

	/**
	 * Creates a new LookAt request using the provided parameters for the mass entity associated to 'ViewerActor', if any.
	 * @param ViewerActor Actor associated to the mass entity that needs to perform the LookAt
	 * @param Priority Priority assigned to the request that will influence selected request when multiple requests are sent to the same entity
	 * @param TargetActor Actor used as reference for the target location. If the actor has an associated mass entity then the target
	 * location will be updated during the request lifetime. Otherwise, the actor location will be used to create a LookAt using a static target location
	 * @param InterpolationSpeed Optional parameter to use predefined interpolation speed
	 * @param CustomInterpolationSpeed Optional parameter to specify a custom interpolation speed when 'InterpolationSpeed' is set to EMassLookAtInterpolationSpeed::Custom
	 * Method will log an error if it fails (e.g., no associated mass entity for viewer)
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "LookAt", meta = (ReturnDisplayName="Request Handle"))
	UE_API FMassLookAtRequestHandle CreateLookAtActorRequest(AActor* ViewerActor
		, FMassLookAtPriority Priority
		, AActor* TargetActor
		, EMassLookAtInterpolationSpeed InterpolationSpeed = EMassLookAtInterpolationSpeed::Regular
		, float CustomInterpolationSpeed = 1.5f) const;

	/**
	 * Removes given request from the active LookAt requests.
	 * Method will log an error if it fails (e.g., invalid request handle).
	 */
	UFUNCTION(BlueprintCallable, BlueprintPure = false, Category = "LookAt")
	UE_API void DeleteRequest(FMassLookAtRequestHandle RequestHandle) const;

	/**
	 * Appends the provided requests to the active ones, then updates the LookAt fragments
	 * of the referenced entity if required.
	 * @param InContext Context that is used to push deferred commands
	 * @param InRequests List of new request to register
	 */
	UE_API void RegisterRequests(const FMassExecutionContext& InContext, TArray<FRequest>&& InRequests);

	/**
	 * Unregisters the provided requests from the list of active ones, then updates the LookAt fragments
	 * of the referenced entity if required.
	 * @param InContext Context that is used to push deferred commands
	 * @param InRequests List of request to unregister
	 */
	UE_API void UnregisterRequests(const FMassExecutionContext& InContext, TConstArrayView<FMassEntityHandle> InRequests);

	/**
	 * Adds new item to the grid for a given entity handle representing a LookAt target.
	 * @param InEntity - Entity used to identify the item in the grid.
	 * @param InTarget - Fragment providing details about the target (i.e., priority)
	 * @param InBounds - Bounding box of the entity.
	 * @return The cell location where the item was added to, can be used later to move or remove the item.
	 */
	[[nodiscard]] UE::Mass::LookAt::FTargetHashGrid2D::FCellLocation AddTarget(const FMassEntityHandle InEntity, const FMassLookAtTargetFragment& InTarget, const FBox& InBounds)
	{
		UE_MT_SCOPED_WRITE_ACCESS(TargetGridAccessDetector);
		return TargetGrid.Add(UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType{InEntity, InTarget.Priority.Get()}, InBounds);
	}

	/**
	 * Moves item based on the entity handle, its previous cell location and its new bounding box.
	 * @param InEntity - Entity used to identify the item in the grid.
	 * @param InTarget - Fragment providing details about the target (i.e., cell location the item was previously added or moved with).
	 * @param InNewBounds - New bounds of the item
	 * @return The cell location where the item is now added to.
	 */
	[[nodiscard]] UE::Mass::LookAt::FTargetHashGrid2D::FCellLocation MoveTarget(const FMassEntityHandle InEntity, const FMassLookAtTargetFragment& InTarget, const FBox& InNewBounds)
	{
		UE_MT_SCOPED_WRITE_ACCESS(TargetGridAccessDetector);
		return TargetGrid.Move(UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType{InEntity, InTarget.Priority.Get()}, InTarget.CellLocation, InNewBounds);
	}

	/**
	 * Moves multiple items based on their entity handle, cell location and new bounding box.
	 * @param InUpdates - List of entity, cell location reference and new bounds to use for each item to update in the grid.
	 * @note Cell location provided in the tuple is a reference and will be updated in the process.
	 */
	void BatchMoveTarget(const TArrayView<TTuple<const FMassEntityHandle, FMassLookAtTargetFragment&, const FBox>> InUpdates)
	{
		UE_MT_SCOPED_WRITE_ACCESS(TargetGridAccessDetector);
		TArray<UE::Mass::LookAt::FTargetHashGrid2D::FCellLocation> UpdatedLocations;
		UpdatedLocations.Reserve(InUpdates.Num());
		for (TTuple<const FMassEntityHandle, FMassLookAtTargetFragment&, const FBox>& UpdateInfo : InUpdates)
		{
			UpdateInfo.Get<1>().CellLocation = TargetGrid.Move(
				/*Item*/UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType{UpdateInfo.Get<0>(), UpdateInfo.Get<1>().Priority.Get()},
				/*OldCellLocation*/UpdateInfo.Get<1>().CellLocation,
				/*NewBounds*/UpdateInfo.Get<2>());
		}
	}

	/**
	 * Removes item based on the entity handle and the cell location it was added with.
	 * @param InEntity - Entity used to identify the item in the grid.
	 * @param InTarget - Fragment providing details about the target (i.e., cell location the item was previously added or moved with).
	 */
	void RemoveTarget(const FMassEntityHandle InEntity, const FMassLookAtTargetFragment& InTarget)
	{
		UE_MT_SCOPED_WRITE_ACCESS(TargetGridAccessDetector);
		TargetGrid.Remove(UE::Mass::LookAt::FTargetHashGrid2D::ItemIDType{InEntity, InTarget.Priority.Get()}, InTarget.CellLocation);
	}

	/**
	 * Returns entity that potentially touch the bounds. Operates on grid level, can have false positives.
	 * @param InQueryBox - Query bounding box.
	 * @param OutEntities - Result of the query, entity handles of potentially overlapping items.
	 */
	template <typename OutT>
	bool Query(const FBox& InQueryBox, OutT& OutEntities) const
	{
		UE_MT_SCOPED_READ_ACCESS(TargetGridAccessDetector);
		TargetGrid.Query(InQueryBox, OutEntities);
		return OutEntities.Num() > 0;
	}

	/** @return Number of entities currently registered in the grid. */
	int32 DebugGetRegisteredTargetCount() const
	{
		return TargetGrid.GetItems().Num();
	}

	/** @return Mass archetype to create entities representing a LookAt request */
	const FMassArchetypeHandle& DebugGetRequestArchetype() const
	{
		return RequestArchetype;
	}

	/** @return Mass archetype to create entities representing a LookAt target */
	const FMassArchetypeHandle& DebugGetTargetArchetype() const
	{
		return TargetArchetype;
	}

#if WITH_MASSGAMEPLAY_DEBUG
	/** @return String detailing all requests registered for a given entity. */
	UE_API FString DebugGetRequestsString(FMassEntityHandle InEntity) const;
#endif

protected:

	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	UE_API virtual TStatId GetStatId() const override;

private:

	/**
	 * Called after registering/unregistering requests to update the LookAtFragment of all viewer entities
	 * referenced in those requests
	 */
	UE_API void UpdateLookAts(const FMassExecutionContext& Context, TConstArrayView<int32> DirtyViewers);

	/** Struct to facilitate processing request per viewer entity. */
	struct FViewerRequest
	{
		FMassEntityHandle Viewer;
		TArray<int32> RequestIndices;
	};

	/** List of all currently registered requests as registered by external systems */
	TArray<FRequest> RegisteredRequests;

	/** Used for lookup in RegisteredRequests, since that array can get large, and linear search would kill performance rather quickly. */
	TMap<FMassEntityHandle, int32> RequestHandleToIndexMap;

	/** List of available indices in ActiveRequests (to preserve stable indices when unregistering requests) */
	TArray<int32> ActiveRequestsFreeList;

	/**
	 * Per viewer entity representation of all the active requests
	 * @todo we never remove elements from this array, which means it can end up being HUGE. Needs fixing.
	 */
	TArray<FViewerRequest> PerViewerRequests;

	/** Used for lookup in PerViewerRequests, since that array can get large, and linear search would kill performance rather quickly. */
	TMap<FMassEntityHandle, int32> ViewerHandleToIndexMap;

	/** Multithread access detector to detect threading issues with any list of requests */
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(RequestsAccessDetector);

	/** Hierarchical 2D hash grid of registered entities representing LookAt targets.*/
	UE::Mass::LookAt::FTargetHashGrid2D TargetGrid;

	/** Multithread access detector to detect threading issues with the hash grid */
	UE_MT_DECLARE_RW_ACCESS_DETECTOR(TargetGridAccessDetector);

	/** Cached archetype to create entities representing a LookAt target */
	FMassArchetypeHandle TargetArchetype;

	/** Cached archetype to create entities representing a LookAt request */
	FMassArchetypeHandle RequestArchetype;
};

template<>
struct TMassExternalSubsystemTraits<UMassLookAtSubsystem>
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false, // hash grid not safe
	};
};

#undef UE_API
