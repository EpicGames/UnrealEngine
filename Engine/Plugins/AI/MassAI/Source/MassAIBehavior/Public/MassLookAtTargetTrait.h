// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "MassLookAtTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassLookAtTargetTrait.generated.h"

UCLASS(meta=(DisplayName="Look At Target"))
class UMassLookAtTargetTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:

	virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	/**
	 * Indicates whether the trait will use an initializer to set target offset using the height of the capsule component if available.
	 */
	UPROPERTY(EditAnywhere, config, Category = LookAt)
	bool bShouldUseCapsuleComponentToSetTargetOffset = true;

	/** Priority assigned to the target to influence target selection */
	UPROPERTY(EditAnywhere, config, Category = LookAt)
	FMassLookAtPriority Priority{static_cast<uint8>(EMassLookAtPriorities::LowestPriority)};
};
