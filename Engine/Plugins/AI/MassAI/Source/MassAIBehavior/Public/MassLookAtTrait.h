// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassLookAtTrait.generated.h"

UCLASS(meta=(DisplayName="Look At"))
class UMassLookAtTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:

	virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};
