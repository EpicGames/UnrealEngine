// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HierarchicalHashGrid2D.h"
#include "Mass/EntityHandle.h"
#include "UObject/ObjectMacros.h"
#include "MassLookAtTypes.generated.h"

namespace UE::Mass::LookAt
{
	constexpr float DefaultCustomInterpolationSpeed = 1.5f;
	constexpr int32 HashGridLevelsOfHierarchy = 2;
	constexpr int32 HashGridRatioBetweenLevels = 4;
	constexpr int32 HashGridResultsSoftLimit = 16;

	struct FTargetHashGridItem
	{
		FTargetHashGridItem(const FMassEntityHandle& TargetEntity, const uint8 Priority)
			: TargetEntity(TargetEntity)
			, Priority(Priority)
		{
		}

		bool operator==(const FTargetHashGridItem& RHS) const
		{
			// We only allow a single entry per entity so no need to compare the priority
			return TargetEntity == RHS.TargetEntity;
		}

		bool operator!=(const FTargetHashGridItem& RHS) const
		{
			return !(*this == RHS);
		}

		FMassEntityHandle TargetEntity;
		uint8 Priority = MAX_uint8;
	};

	using FTargetHashGrid2D = THierarchicalHashGrid2D<HashGridLevelsOfHierarchy, HashGridRatioBetweenLevels, FTargetHashGridItem>;
}

/**
 * Enum representing the different interpolation speeds that can be used when assigning new LookAt targets
 */
UENUM(BlueprintType)
enum class EMassLookAtInterpolationSpeed : uint8
{
	Instant,
	Fast,
	Regular,
	Slow,
	Custom
};

/**
 * Enum used to define the number of configurable priorities exposed by the MassLookAtSettings
 */
UENUM()
enum class EMassLookAtPriorities : uint8
{
	MaxPriorities = 16,
	MaxPriorityIndex = MaxPriorities - 1,
	LowestPriority = MaxPriorityIndex
};

/**
 * Struct used as a priority selector exposed to the Editor
 */
USTRUCT(BlueprintType)
struct FMassLookAtPriority
{
	GENERATED_BODY()

	FMassLookAtPriority() = default;
	
	explicit FMassLookAtPriority(const uint8 InBit)
		: Value(InBit)
	{
		check(InBit <= static_cast<uint8>(EMassLookAtPriorities::MaxPriorities));
	}

	void Set(const uint8 InBit)
	{
		check(InBit <= static_cast<uint8>(EMassLookAtPriorities::MaxPriorities));
		Value = InBit;
	}

	uint8 Get() const
	{
		return Value;
	}

	void Reset()
	{
		Value = NoneValue;
	}

	bool IsValid() const
	{
		return Value != NoneValue;
	}

	bool operator==(const FMassLookAtPriority& RHS) const
	{
		return Value == RHS.Value;
	}

	bool operator!=(const FMassLookAtPriority& RHS) const
	{
		return Value != RHS.Value;
	}

private:
	static constexpr uint8 NoneValue = 0xFF;

	UPROPERTY(Category = LookAt, EditAnywhere)
	uint8 Value = NoneValue;
};

/**
 * Struct used to represent configurable priorities in MassLookAtSettings
 */
USTRUCT()
struct FMassLookAtPriorityInfo
{
	GENERATED_BODY()

	bool IsValid() const
	{
		return !Name.IsNone() && Priority.IsValid();
	}

	UPROPERTY(Category = LookAt, EditAnywhere)
	FName Name;

	UPROPERTY(Category = LookAt, EditAnywhere)
	FMassLookAtPriority Priority;
};