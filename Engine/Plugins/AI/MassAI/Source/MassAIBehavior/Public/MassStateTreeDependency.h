// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityConcepts.h"
#include "MassRequirements.h"
#include "Misc/NotNull.h"
#include "MassStateTreeDependency.generated.h"

/** A dependency the state tree has on other fragment or system used by mass. */
USTRUCT()
struct FMassStateTreeDependency
{
	GENERATED_BODY()

	FMassStateTreeDependency() = default;
	FMassStateTreeDependency(TNotNull<const UStruct*> InType, EMassFragmentAccess InAccess)
		: Type(InType)
		, Access(InAccess)
	{}

	UPROPERTY()
	TObjectPtr<const UStruct> Type;

	UPROPERTY()
	EMassFragmentAccess Access = EMassFragmentAccess::None;
};

namespace UE::MassBehavior
{
	
/**
 * FStateTreeDependencyBuilder is a utility struct to build a list of dependencies used by the state tree.
 * The add dependency can be chain for ease of use.
 * Example Usage:
 * {
 *		FStateTreeDependencyBuilder Builder()
 *		.AddReadOnly<FTransformFragment>()
 *		.AddReadWrite(ComponentHitSubsystemHandle);
 *		Builder.AddReadOnly<FMassZoneGraphAnnotationFragment>();
 * } 
 */
struct FStateTreeDependencyBuilder 
{
public:
	enum class EAccessType : uint8
	{
		ReadOnly = (uint8)EMassFragmentAccess::ReadOnly,
		ReadWrite = (uint8)EMassFragmentAccess::ReadWrite,
	};

public:
	MASSAIBEHAVIOR_API FStateTreeDependencyBuilder(TArray<FMassStateTreeDependency>& Dependencies);

	/** Add a read only dependency. */
	template<typename T>
	FStateTreeDependencyBuilder& AddReadOnly() requires TIsDerivedFrom<T, UObject>::IsDerived
	{
		Add(T::StaticClass(), EAccessType::ReadOnly);
		return *this;
	}

	/** Add a read only dependency. */
	template<UE::Mass::CNonTag T>
	FStateTreeDependencyBuilder& AddReadOnly()
	{
		Add(T::StaticStruct(), EAccessType::ReadOnly);
		return *this;
	}

	/** Add a read only dependency from a handle. */
	template<typename T>
	FStateTreeDependencyBuilder& AddReadOnly(const T& /*Handle*/) requires TIsDerivedFrom<typename T::DataType, UObject>::IsDerived
	{
		Add(T::DataType::StaticClass(), EAccessType::ReadOnly);
		return *this;
	}
	
	/** Add a read only dependency from a handle. */
	template<typename T>
	FStateTreeDependencyBuilder& AddReadOnly(const T& /*Handle*/) requires UE::Mass::CNonTag<typename T::DataType>
	{
		Add(T::DataType::StaticStruct(), EAccessType::ReadOnly);
		return *this;
	}

	/** Add a read write dependency. */
	template<typename T>
	FStateTreeDependencyBuilder& AddReadWrite() requires TIsDerivedFrom<T, UObject>::IsDerived
	{
		Add(T::StaticClass(), EAccessType::ReadWrite);
		return *this;
	}
	
	/** Add a read write dependency. */
	template<UE::Mass::CNonTag T>
	FStateTreeDependencyBuilder& AddReadWrite()
	{
		Add(T::StaticStruct(), EAccessType::ReadWrite);
		return *this;
	}

	/** Add a read write dependency from a handle. */
	template<typename T>
	FStateTreeDependencyBuilder& AddReadWrite(const T& /*Handle*/) requires TIsDerivedFrom<typename T::DataType, UObject>::IsDerived
	{
		Add(T::DataType::StaticClass(), EAccessType::ReadWrite);
		return *this;
	}

	/** Add a read write dependency from a handle. */
	template<typename T>
	FStateTreeDependencyBuilder& AddReadWrite(const T& /*Handle*/) requires UE::Mass::CNonTag<typename T::DataType>
	{
		Add(T::DataType::StaticStruct(), EAccessType::ReadWrite);
		return *this;
	}

	/** Add a dependency for a specific type. */
	MASSAIBEHAVIOR_API void Add(TNotNull<const UStruct*> Struct, EAccessType Access);

	/** @return the list of dependencies. */
	const TConstArrayView<FMassStateTreeDependency> GetDependencies() const
	{
		return Dependencies;
	}

private:
	/** The list of dependencies. */
	TArray<FMassStateTreeDependency>& Dependencies;
};

}
