// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/EntityHandle.h"
#include "MassExecutionContext.h"
#include "StateTreeExecutionContext.h"
#include "MassStateTreeExecutionContext.generated.h"

struct FMassExecutionContext;
struct FMassEntityManager;
class UMassSignalSubsystem;
struct FMassCommandBuffer;

USTRUCT()
struct FMassExecutionExtension : public FStateTreeExecutionExtension
{
	GENERATED_BODY()

public:
	virtual FString GetInstanceDescription(const FContextParameters& Context) const override;
	virtual void OnLinkedStateTreeOverridesSet(const FContextParameters& Context, const FStateTreeReferenceOverrides& Overrides) override;

	FMassEntityHandle Entity;
	uint32 LinkedStateTreeOverridesHash = 0;
};

/**
 * Extends FStateTreeExecutionContext to provide additional data to Evaluators and Tasks related to MassSimulation
 */
struct FMassStateTreeExecutionContext : public FStateTreeExecutionContext 
{
	MASSAIBEHAVIOR_API FMassStateTreeExecutionContext(UObject& InOwner
		, const UStateTree& InStateTree
		, FStateTreeInstanceData& InInstanceData
		, FMassExecutionContext& InContext);

	UE_DEPRECATED(5.6, "Use the other constructor that doesn't require MassEntityManager and MassSignalSubSystem")
	FMassStateTreeExecutionContext(UObject& InOwner
		, const UStateTree& InStateTree
		, FStateTreeInstanceData& InInstanceData
		, FMassEntityManager& InEntityManager
		, UMassSignalSubsystem& InSignalSubsystem
		, FMassExecutionContext& InContext)
			: FMassStateTreeExecutionContext(InOwner, InStateTree, InInstanceData, InContext)
	{
	}

	/** Start executing. */
	MASSAIBEHAVIOR_API EStateTreeRunStatus Start();
	MASSAIBEHAVIOR_API EStateTreeRunStatus Start(const FInstancedPropertyBag* InitialParameters, int32 RandomSeed = -1);

	FMassEntityManager& GetEntityManager() const
	{
		return GetMassEntityExecutionContext().GetEntityManagerChecked();
	}

	FMassExecutionContext& GetMassEntityExecutionContext() const
	{
		return *MassEntityExecutionContext;
	}

	FMassEntityHandle GetEntity() const
	{
		return Entity;
	}

	MASSAIBEHAVIOR_API void SetEntity(const FMassEntityHandle InEntity);

protected:
	MASSAIBEHAVIOR_API virtual void BeginDelayedTransition(const FStateTreeTransitionDelayedState& DelayedState) override;
	FMassExecutionContext* MassEntityExecutionContext = nullptr;
	FMassEntityHandle Entity;
};
