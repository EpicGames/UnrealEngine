// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassStateTreeSubsystem.h"
#include "MassStateTreeFragments.generated.h"


USTRUCT()
struct FMassStateTreeInstanceFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassStateTreeInstanceFragment() = default;

	/** Handle to a StateTree instance data in MassStateTreeSubsystem. */
	FMassStateTreeInstanceHandle InstanceHandle;

	/** The last update time use to calculate ticking delta time. */
	double LastUpdateTimeInSeconds = 0.;
};


USTRUCT()
struct FMassStateTreeSharedFragment : public FMassConstSharedFragment
{
	GENERATED_BODY()

	FMassStateTreeSharedFragment() = default;

	UPROPERTY()
	TObjectPtr<UStateTree> StateTree = nullptr;
};
