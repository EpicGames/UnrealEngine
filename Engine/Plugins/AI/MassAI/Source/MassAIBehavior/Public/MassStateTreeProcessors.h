// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSignalProcessorBase.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassStateTreeFragments.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassObserverProcessor.h"
#include "MassProcessorDependencySolver.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassLODTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassStateTreeProcessors.generated.h"

struct FMassStateTreeExecutionContext;
struct FMassSubsystemRequirements;
struct FMassFragmentRequirements;
class UStateTree;

/** 
 * Processor to stop and uninitialize StateTrees on entities.
 */
UCLASS(MinimalAPI)
class UMassStateTreeFragmentDestructor : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	MASSAIBEHAVIOR_API UMassStateTreeFragmentDestructor();

protected:
	MASSAIBEHAVIOR_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;

	UPROPERTY(Transient)
	TObjectPtr<UMassSignalSubsystem> SignalSubsystem = nullptr;
};

/**
 * Special tag to know if the state tree has been activated
 */
USTRUCT()
struct FMassStateTreeActivatedTag : public FMassTag
{
	GENERATED_BODY()
};
/**
 * Processor to send the activation signal to the state tree which will execute the first tick */
UCLASS(MinimalAPI)
class UMassStateTreeActivationProcessor : public UMassProcessor
{
	GENERATED_BODY()
public:
	MASSAIBEHAVIOR_API UMassStateTreeActivationProcessor();
protected:
	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;
	
	FMassEntityQuery EntityQuery;
};

/** 
 * The processor that the UMassStateTreeSubsystem will instantiate for every unique StateTree Mass-requirements.
 * The user is not expected to instantiate these processors manually, but a project-specific extension can be implemented.
 * It needs to derive from UMassStateTreeProcessor and set as the value of UMassStateTreeSubsystem.DynamicProcessorClass.
 */
UCLASS(MinimalAPI)
class UMassStateTreeProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()

public:
	MASSAIBEHAVIOR_API UMassStateTreeProcessor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

	/**
	 * Called to configure dynamic processor's additional requirements that will ensure its located
	 * properly within Mass's processing graph. Calling this function is allowed only until the
	 * processor is Initialized. The function will ensure that's the case. 
	 */
	MASSAIBEHAVIOR_API void SetExecutionRequirements(const FMassFragmentRequirements& FragmentRequirements, const FMassSubsystemRequirements& SubsystemRequirements);

	/**
	 * Adds StateTree to the collection of the assets this specific processor instance will handle. 
	 */
	MASSAIBEHAVIOR_API void AddHandledStateTree(TNotNull<const UStateTree*> StateTree);

protected:
	MASSAIBEHAVIOR_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSAIBEHAVIOR_API virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& EntitySignals) override;

	MASSAIBEHAVIOR_API virtual void ExportRequirements(FMassExecutionRequirements& OutRequirements) const override;

	/**
	 * Stores the additional requirements as configured by SetExecutionRequirements.
	 * These requirements ensure the processor will be placed at the right location in the processing graph
	 * to avoid data races.
	 */
	FMassExecutionRequirements ExecutionRequirements;

	/** The assets handled by this processor - entities utilizing any of these assets will be processed by this processor */
	UPROPERTY()
	TArray<TObjectPtr<const UStateTree>> HandledStateTrees;

	/** Configures whether parallel update for FMassArchetypeChunks should be used instead of the default single threaded update (i.e., ParallelForEachEntityChunk instead of ForEachEntityChunk). */
	UPROPERTY(EditDefaultsOnly, Category = Processor, config)
	bool bProcessEntitiesInParallel = false;
};
