// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "StateTreeSchema.h"
#include "MassStateTreeDependency.h"
#include "MassStateTreeSchema.generated.h"

#define UE_API MASSAIBEHAVIOR_API

/**
 * StateTree for Mass behaviors.
 */
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, CollapseCategories, meta = (DisplayName = "Mass Behavior", CommonSchema))
class UMassStateTreeSchema : public UStateTreeSchema
{
	GENERATED_BODY()

public:
	/** Fetches a read-only view of the Mass-relevant requirements of the associated StateTee */
	const TArray<FMassStateTreeDependency>& GetDependencies() const;

#if WITH_EDITOR
	virtual bool AllowQueuedCompilation() const override
	{
		// If queued, they might be compiled in a mass worker thread via CompileIfNeededSynchronously.
		return false;
	}
#endif

protected:
	UE_API virtual bool IsStructAllowed(const UScriptStruct* InScriptStruct) const override;
	UE_API virtual bool IsExternalItemAllowed(const UStruct& InStruct) const override;
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;

protected:
	/**
	 * The Mass-relevant requirements of the associated StateTee, collected during StateTree's linking
	 * @see UStateTree::Link
	 * @see UMassStateTreeSchema::Link
	 */
	UPROPERTY(Transient)
	TArray<FMassStateTreeDependency> Dependencies;
};

//-----------------------------------------------------------------------------
// INLINE
//-----------------------------------------------------------------------------
inline const TArray<FMassStateTreeDependency>& UMassStateTreeSchema::GetDependencies() const
{
	return Dependencies;
}

#undef UE_API
