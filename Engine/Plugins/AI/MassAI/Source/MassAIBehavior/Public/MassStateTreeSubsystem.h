// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "MassSubsystemBase.h"
#include "StateTreeExecutionContext.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Mass/ExternalSubsystemTraits.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_7
#include "Misc/MTTransactionallySafeAccessDetector.h"
#include "UObject/ObjectKey.h"
#include "MassStateTreeSubsystem.generated.h"

#define UE_API MASSAIBEHAVIOR_API

class UStateTree;
class UMassStateTreeProcessor;
class UMassSimulationSubsystem;
struct FMassEntityManager;

namespace UE::Mass::StateTree
{
	extern bool bDynamicSTProcessorsEnabled;
}

USTRUCT()
struct FMassStateTreeInstanceDataItem
{
	GENERATED_BODY()

	UPROPERTY()
	FStateTreeInstanceData InstanceData;

	UPROPERTY()
	int32 Generation = 0;
};

/**
* A subsystem managing StateTree assets in Mass
*/
UCLASS(MinimalAPI)
class UMassStateTreeSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()
	
protected:
	// USubsystem BEGIN
	UE_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	// USubsystem END

public:
	/**
	 * Allocates new instance data for specified StateTree.
	 * @param StateTree StateTree to allocated the data for.
	 * @return Handle to the data.
	 */
	UE_API FMassStateTreeInstanceHandle AllocateInstanceData(const UStateTree* StateTree);

	/**
	 * Frees instance data.
	 * @param Handle Instance data handle to free.
	 */
	UE_API void FreeInstanceData(const FMassStateTreeInstanceHandle Handle);

	/** @return Pointer to instance data held by the handle, or nullptr if handle is not valid. */
	FStateTreeInstanceData* GetInstanceData(const FMassStateTreeInstanceHandle Handle)
	{
		UE_MT_SCOPED_READ_ACCESS(InstanceDataMTDetector);
		return IsValidHandle(Handle) ? &InstanceDataArray[Handle.GetIndex()].InstanceData : nullptr;
	}

	/** @return True if the handle points to active instance data. */
	bool IsValidHandle(const FMassStateTreeInstanceHandle Handle) const
	{
		UE_MT_SCOPED_READ_ACCESS(InstanceDataMTDetector);
		return InstanceDataArray.IsValidIndex(Handle.GetIndex()) && InstanceDataArray[Handle.GetIndex()].Generation == Handle.GetGeneration();
	}
	
protected:

	/**
	 * Gathers Mass-relevant processing requirements from StateTree and spawns
	 * a dynamic processor to handle entities using this given asset
	 */
	UE_API void CreateProcessorForStateTree(TNotNull<const UStateTree*> StateTree);

	TArray<int32> InstanceDataFreelist;

	UPROPERTY(Transient)
	TArray<FMassStateTreeInstanceDataItem> InstanceDataArray;

	/**
	 * Multithread access detector to prevent attempts to allocate new instance data
	 * from multiple threads, which is not supported by the current implementation.
	 * @see AllocateInstanceData, FreeInstanceData, CreateProcessorForStateTree
	 * @todo Instance data creation needs to be refactored in order to allow parallelization of StateTree activations
	 */
	UE_MT_DECLARE_TS_RW_ACCESS_DETECTOR(InstanceDataMTDetector);

	/**
	 * The relevant Entity Manager. Needed to build processing requirements for dynamic processors.
	 */
	TSharedPtr<FMassEntityManager> ĘntityManager;

	/**
	 * The key represents a hash of mass requirements calculated from a StateTree assets.
	 * Using the hash rather than a StateTree asset pointer since multiple assets can have identical requirements,
	 * and we want just one dynamic processor to handle all of them.
	 * @note that might change if in practice it turns out that runtime entity chunk filtering turns out to be
	 *	too expensive when multiple ST assets are used.
	 */
	UPROPERTY()
	TMap<uint32, TObjectPtr<UMassStateTreeProcessor>> RequirementsHashToProcessor;

	// @todo ask Patrick how would this behave when ST asset gets changed/recompiled or whatnot
	/**
	 * Mapping StateTree assets to the dynamic processors handling them. Note that it's not 1:1, a single processor can handle multiple assets.
	 */
	TMap<TObjectKey<UStateTree>, TObjectPtr<UMassStateTreeProcessor>> StateTreeToProcessor;

	/** Cached SimulationSubsystem for registering dynamic processors. */
	UPROPERTY()
	TObjectPtr<UMassSimulationSubsystem> SimulationSubsystem;

	/**
	 * Class to use when creating dynamic processors to handle given StateTree assets.
	 * Set based on UMassBehaviorSettings.DynamicStateTreeProcessorClass
	 */
	UPROPERTY(Transient)
	TSubclassOf<UMassStateTreeProcessor> DynamicProcessorClass;
};

#undef UE_API
