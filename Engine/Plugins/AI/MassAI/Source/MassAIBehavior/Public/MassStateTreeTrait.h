// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTraitBase.h"
#include "MassStateTreeTrait.generated.h"

#define UE_API MASSAIBEHAVIOR_API

class UStateTree;

/**
 * Feature that adds StateTree execution functionality to a mass agent.
 */
UCLASS(MinimalAPI, meta=(DisplayName="StateTree"))
class UMassStateTreeTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:

	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
	UE_API virtual bool ValidateTemplate(const FMassEntityTemplateBuildContext& BuildContext, const UWorld& World, FAdditionalTraitRequirements& OutTraitRequirements) const override;

	UPROPERTY(Category="StateTree", EditAnywhere, meta=(RequiredAssetDataTags="Schema=/Script/MassAIBehavior.MassStateTreeSchema"))
	TObjectPtr<UStateTree> StateTree;
};

#undef UE_API
