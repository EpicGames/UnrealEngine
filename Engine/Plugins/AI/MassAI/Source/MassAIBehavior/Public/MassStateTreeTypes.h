// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessingTypes.h"
#include "StateTreeConditionBase.h"
#include "StateTreeEvaluatorBase.h"
#include "StateTreePropertyFunctionBase.h"
#include "StateTreeTaskBase.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassStateTreeTypes.generated.h"

enum class EProcessorExecutionFlags : uint8;
namespace UE::MassBehavior
{
struct FStateTreeDependencyBuilder;
}

namespace UE::MassStateTree
{
/** Flags to indicates to the MassStateTree processors and traits in which contexts they can execute (e.g., Standalone, Server, etc.) */
static constexpr EProcessorExecutionFlags ExecutionFlags(EProcessorExecutionFlags::Standalone | EProcessorExecutionFlags::Server);;
}

/**
 * Signals used by the StateTree framework in Mass
 */
namespace UE::Mass::Signals
{
	const FName StateTreeActivate = FName(TEXT("StateTreeActivate"));
	const FName LookAtFinished = FName(TEXT("LookAtFinished"));
	const FName NewStateTreeTaskRequired = FName(TEXT("NewStateTreeTaskRequired"));
	const FName StandTaskFinished = FName(TEXT("StandTaskFinished"));
	const FName AnimateTaskFinished = FName(TEXT("AnimateTaskFinished"));
	const FName DelayedTransitionWakeup = FName(TEXT("DelayedTransitionWakeup"));
	// @todo MassStateTree: move this to its game plugin when possible
	const FName ContextualAnimTaskFinished = FName(TEXT("ContextualAnimTaskFinished"));
}

/**
 * Base struct for all Mass StateTree Evaluators.
 */
USTRUCT(meta = (Hidden, DisplayName = "Mass Evaluator Base"))
struct FMassStateTreeEvaluatorBase : public FStateTreeEvaluatorBase
{
	GENERATED_BODY()

	/**
	 * Appends this evaluator's Mass dependencies to the given Builder.
	 * This is done once for every evaluator instance, when the state tree asset is loaded or compiled.
	 */
	virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
	{
	}
};

/**
 * Base struct for all Mass StateTree Tasks.
 */
USTRUCT(meta = (Hidden, DisplayName = "Mass Task Base"))
struct FMassStateTreeTaskBase : public FStateTreeTaskBase
{
	GENERATED_BODY()

	/**
	 * Appends this task's Mass dependencies to the given Builder.
	 * This is done once for every task instance, when the state tree asset is loaded or compiled.
	 */
	virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
	{
	}
};

/**
 * Base struct for all Mass StateTree Conditions.
 */
USTRUCT(meta = (Hidden, DisplayName = "Mass Condition Base"))
struct FMassStateTreeConditionBase : public FStateTreeConditionBase
{
	GENERATED_BODY()

	/**
	 * Appends this condition's Mass dependencies to the given Builder.
	 * This is done once for every condition instance, when the state tree asset is loaded or compiled.
	 */
	virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
	{
	}
};

/**
 * Base struct for all Mass StateTree Property Functions.
 */
USTRUCT(meta = (Hidden, DisplayName = "Mass Property Function Base"))
struct FMassStateTreePropertyFunctionBase : public FStateTreePropertyFunctionBase
{
	GENERATED_BODY()

	/**
	 * Appends this property function's Mass dependencies to the given Builder.
	 * This is done once for every property function instance, when the state tree asset is loaded or compiled.
	 */
	virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const
	{
	}
};

/**
 * A handle pointing to a StateTree instance data in UMassStateTreeSubsystem.
 */
struct FMassStateTreeInstanceHandle
{
	FMassStateTreeInstanceHandle() = default;

	/** Initializes new handle based on an index */
	static FMassStateTreeInstanceHandle Make(const int32 InIndex, const int32 InGeneration) { return FMassStateTreeInstanceHandle(InIndex, InGeneration); }
	
	/** @returns index the handle points to */
	int32 GetIndex() const { return Index; }

	/** @returns generation of the handle, used to identify recycled indices. */ 
	int32 GetGeneration() const { return Generation; }

	/** @returns true if the handle is valid. */
	bool IsValid() const { return Index != INDEX_NONE; }

protected:
	FMassStateTreeInstanceHandle(const int32 InIndex, const int32 InGeneration) : Index(InIndex), Generation(InGeneration) {}

	int32 Index = INDEX_NONE;
	int32 Generation = 0;
};
