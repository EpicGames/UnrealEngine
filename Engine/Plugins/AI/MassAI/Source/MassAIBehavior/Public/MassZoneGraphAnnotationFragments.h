// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "ZoneGraphTypes.h"
#include "MassZoneGraphAnnotationFragments.generated.h"

struct FMassExecutionContext;

USTRUCT()
struct FMassZoneGraphAnnotationFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Behavior tags for current lane */
	UPROPERTY()
	FZoneGraphTagMask Tags;
};

USTRUCT()
struct FMassZoneGraphAnnotationVariableTickChunkFragment : public FMassChunkFragment
{
	GENERATED_BODY();

	/** Update the ticking frequency of the chunk and return if this chunk should be process this frame */
	static bool UpdateChunk(FMassExecutionContext& Context);

	float TimeUntilNextTick = 0.0f;
	bool bInitialized = false;
};
