// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassSignalProcessorBase.h"
#include "MassObserverProcessor.h"
#include "MassZoneGraphAnnotationProcessors.generated.h"

#define UE_API MASSAIBEHAVIOR_API

class UMassSignalSubsystem;
class UZoneGraphAnnotationSubsystem;
struct FMassZoneGraphAnnotationFragment;
struct FMassZoneGraphLaneLocationFragment;
struct FMassEntityHandle;

/** 
 * Processor for initializing ZoneGraph annotation tags.
 */
UCLASS(MinimalAPI)
class UMassZoneGraphAnnotationTagsInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassZoneGraphAnnotationTagsInitializer();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

/** 
 * Processor for update ZoneGraph annotation tags periodically and on lane changed signal.
 */
UCLASS(MinimalAPI)
class UMassZoneGraphAnnotationTagUpdateProcessor : public UMassSignalProcessorBase
{
	GENERATED_BODY()

public:
	UE_API UMassZoneGraphAnnotationTagUpdateProcessor();
	
protected:
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	UE_API void UpdateAnnotationTags(UZoneGraphAnnotationSubsystem& ZoneGraphAnnotationSubsystem, FMassZoneGraphAnnotationFragment& AnnotationTags, const FMassZoneGraphLaneLocationFragment& LaneLocation, FMassEntityHandle Entity);

	UE_API virtual void SignalEntities(FMassEntityManager& EntityManager, FMassExecutionContext& Context, FMassSignalNameLookup& /*Unused*/) override;

	// Frame buffer, it gets reset every frame.
	TArray<FMassEntityHandle> TransientEntitiesToSignal;
};

#undef UE_API
