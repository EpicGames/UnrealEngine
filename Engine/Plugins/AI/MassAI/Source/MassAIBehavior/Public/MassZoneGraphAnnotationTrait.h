// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassZoneGraphAnnotationTrait.generated.h"

#define UE_API MASSAIBEHAVIOR_API

UCLASS(MinimalAPI, meta = (DisplayName = "ZoneGraph Annotation"))
class UMassZoneGraphAnnotationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

#undef UE_API
