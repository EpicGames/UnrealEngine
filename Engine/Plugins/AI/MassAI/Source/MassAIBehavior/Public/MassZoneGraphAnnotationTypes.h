// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace UE::Mass::Signals
{
	const FName AnnotationTagsChanged = FName(TEXT("AnnotationTagsChanged"));
} // UE::Mass::Signals
