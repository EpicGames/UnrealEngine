// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassSmartObjectRequest.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassStateTreeTypes.h"
#include "SmartObjectRuntime.h"
#include "StateTreePropertyRef.h"
#include "MassClaimSmartObjectTask.generated.h"

struct FStateTreeExecutionContext;
struct FMassSmartObjectUserFragment;
class USmartObjectSubsystem;
class UMassSignalSubsystem;
struct FTransformFragment;
struct FMassMoveTargetFragment;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};
namespace UE::Mass::SmartObject
{
	struct FMRUSlotsFragment;
}

/**
 * Tasks to claim a smart object from search results and release it when done.
 */
USTRUCT()
struct FMassClaimSmartObjectTaskInstanceData
{
	GENERATED_BODY()

	/** Result of the candidates search request (Input) */
	UPROPERTY(VisibleAnywhere, Category = Input, meta = (RefType = "/Script/MassSmartObjects.MassSmartObjectCandidateSlots"))
	FStateTreePropertyRef CandidateSlots;

	UPROPERTY(VisibleAnywhere, Category = Output)
	FSmartObjectClaimHandle ClaimedSlot;

	UPROPERTY(VisibleAnywhere, Category = Output)
	FTransform ClaimedSlotTransform;
};

USTRUCT(meta = (DisplayName = "Claim SmartObject"))
struct FMassClaimSmartObjectTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassClaimSmartObjectTaskInstanceData;

	MASSAIBEHAVIOR_API FMassClaimSmartObjectTask();

protected:
	MASSAIBEHAVIOR_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	MASSAIBEHAVIOR_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	MASSAIBEHAVIOR_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	MASSAIBEHAVIOR_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	MASSAIBEHAVIOR_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<FMassSmartObjectUserFragment> SmartObjectUserHandle;
	TStateTreeExternalDataHandle<UE::Mass::SmartObject::FMRUSlotsFragment, EStateTreeExternalDataRequirement::Optional> SmartObjectMRUSlotsHandle;
	TStateTreeExternalDataHandle<USmartObjectSubsystem> SmartObjectSubsystemHandle;
	TStateTreeExternalDataHandle<UMassSignalSubsystem> MassSignalSubsystemHandle;

	/** Delay in seconds before trying to use another smart object */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float InteractionCooldown = 0.f;
};
