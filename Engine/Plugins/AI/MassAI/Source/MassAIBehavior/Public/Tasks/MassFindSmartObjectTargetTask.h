// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassNavigationTypes.h"
#include "MassStateTreeTypes.h"
#include "SmartObjectRuntime.h"
#include "SmartObjectSubsystem.h"
#include "MassFindSmartObjectTargetTask.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FTransformFragment;
struct FAgentRadiusFragment;
struct FAgentHeightFragment;
class USmartObjectSubsystem;

USTRUCT()
struct FMassFindSmartObjectTargetInstanceData
{
	GENERATED_BODY()

	/**
	 * When using the entrance location request with selection method 'NearestToSearchLocation',
	 * this property indicates whether the request will set the search location using
	 * the entity transform or use the request value (can be bound).
	 */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bUseEntityLocationAsSearchLocation = true;

	/** Request parameters when using the entrance location request */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FSmartObjectSlotEntranceLocationRequest EntranceRequest;

	UPROPERTY(VisibleAnywhere, Category = Input)
	FSmartObjectClaimHandle ClaimedSlot;

	UPROPERTY(EditAnywhere, Category = Output)
	FMassTargetLocation SmartObjectLocation;
};

/**
 * Computes move target to a smart object based on current location.
 * The move target will use the entrance location found by the EntranceRequest if possible,
 * otherwise the slot location will be used.
 */
USTRUCT(meta = (DisplayName = "Find Smart Object Target"))
struct FMassFindSmartObjectTargetTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassFindSmartObjectTargetInstanceData;

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;

	TStateTreeExternalDataHandle<FAgentRadiusFragment, EStateTreeExternalDataRequirement::Optional> AgentRadiusHandle;
	TStateTreeExternalDataHandle<FAgentHeightFragment, EStateTreeExternalDataRequirement::Optional> AgentHeightHandle;
	TStateTreeExternalDataHandle<FTransformFragment> TransformHandle;
	TStateTreeExternalDataHandle<USmartObjectSubsystem> SmartObjectSubsystemHandle;

	/**
	 * Whether the entrance location request should be used first to find an entrance location
	 * to use instead of the slot location. Slot location will be used if not entrances can be found.
	 */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bUseEntranceLocationRequest = true;
};

#undef UE_API
