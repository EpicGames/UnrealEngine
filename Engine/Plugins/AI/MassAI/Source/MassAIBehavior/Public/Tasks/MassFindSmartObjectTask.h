// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "MassStateTreeTypes.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "SmartObjectSubsystem.h"
#include "MassSignalSubsystem.h"
#endif
#include "MassSmartObjectRequest.h"
#include "MassFindSmartObjectTask.generated.h"

class UMassSignalSubsystem;
class USmartObjectSubsystem;
struct FMassSmartObjectUserFragment;
struct FMassZoneGraphLaneLocationFragment;
struct FTransformFragment;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};
namespace UE::Mass::SmartObject
{
	struct FMRUSlotsFragment;
};

USTRUCT()
struct FMassFindSmartObjectTaskInstanceData
{
	GENERATED_BODY()

	/** Result of the candidates search request */
	UPROPERTY(VisibleAnywhere, Category = Output)
	FMassSmartObjectCandidateSlots FoundCandidateSlots;		// @todo: Should turn this in a StateTree result/value.

	UPROPERTY(VisibleAnywhere, Category = Output)
	bool bHasCandidateSlots = false;

	/** The identifier of the search request send by the task to find candidates */
	UPROPERTY()
	FMassSmartObjectRequestID SearchRequestID;

	/** Next update time; task will not do anything when Tick gets called before that time */
	UPROPERTY()
	double NextUpdate = 0.;

	/** Last lane where the smart objects were searched. */
	UPROPERTY()
	FZoneGraphLaneHandle LastLane;
};

USTRUCT(meta = (DisplayName = "Find Smart Object"))
struct FMassFindSmartObjectTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassFindSmartObjectTaskInstanceData;

	MASSAIBEHAVIOR_API FMassFindSmartObjectTask();

protected:
	MASSAIBEHAVIOR_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	MASSAIBEHAVIOR_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	MASSAIBEHAVIOR_API virtual void StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const override;
	MASSAIBEHAVIOR_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	MASSAIBEHAVIOR_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<USmartObjectSubsystem> SmartObjectSubsystemHandle;
	TStateTreeExternalDataHandle<UMassSignalSubsystem> MassSignalSubsystemHandle;
	TStateTreeExternalDataHandle<FTransformFragment> EntityTransformHandle;
	TStateTreeExternalDataHandle<FMassSmartObjectUserFragment> SmartObjectUserHandle;
	TStateTreeExternalDataHandle<UE::Mass::SmartObject::FMRUSlotsFragment, EStateTreeExternalDataRequirement::Optional> SmartObjectMRUSlotsHandle;
	TStateTreeExternalDataHandle<FMassZoneGraphLaneLocationFragment, EStateTreeExternalDataRequirement::Optional> LocationHandle;

	/** Gameplay tag query for finding matching smart objects. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FGameplayTagQuery ActivityRequirements;

	/** How frequently to search for new candidates. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float SearchInterval = 10.0f;
	
	/** If true, search smart objects using current lane position else, use world position. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bFindFromLaneLocation = true;
};
