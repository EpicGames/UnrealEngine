// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassStateTreeTypes.h"
#include "MassLookAtFragments.h"
#include "MassLookAtTask.generated.h"

#define UE_API MASSAIBEHAVIOR_API

class UMassSignalSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
 * Task to assign a LookAt target for mass processing
 */
USTRUCT()
struct FMassLookAtTaskInstanceData
{
	GENERATED_BODY()

	/** Entity to set as the target for the LookAt behavior. */
	UPROPERTY(EditAnywhere, Category = Input, meta = (Optional))
	FMassEntityHandle TargetEntity;
 
	/** Delay before the task ends. Default (0 or any negative) will run indefinitely so it requires a transition in the state tree to stop it. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float Duration = 0.f;

	/** Accumulated time used to stop task if duration is set */
	UPROPERTY()
	float Time = 0.f;
};

USTRUCT(meta = (DisplayName = "Mass LookAt Task"))
struct FMassLookAtTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassLookAtTaskInstanceData;
	
protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	bool TryActivateSystemicLookAt(const FStateTreeExecutionContext& Context, const FInstanceDataType& InstanceData, FMassLookAtFragment& Fragment) const;

	TStateTreeExternalDataHandle<UMassSignalSubsystem> MassSignalSubsystemHandle;
	TStateTreeExternalDataHandle<FMassLookAtFragment, EStateTreeExternalDataRequirement::Optional> LookAtHandle;

	/** Look At Priority */
	UPROPERTY(EditAnywhere, Category = Parameter)
	FMassLookAtPriority Priority{static_cast<uint8>(EMassLookAtPriorities::LowestPriority)};

	/** Look At Mode */
	UPROPERTY(EditAnywhere, Category = Parameter)
	EMassLookAtMode LookAtMode = EMassLookAtMode::LookForward;

	/** Look at interpolation speed (not used by the LookAt processor but can be forwarded to the animation system). */
	UPROPERTY(EditAnywhere, Category = Parameter)
	EMassLookAtInterpolationSpeed InterpolationSpeed = EMassLookAtInterpolationSpeed::Regular;

	/**
	 * Look at custom interpolation speed used when 'InterpolationSpeed = EMassLookAtInterpolationSpeed::Custom'
	 * (not used by the LookAt processor but can be forwarded to the animation system).
	 */
	UPROPERTY(EditAnywhere, Category = Parameter, meta = (EditCondition="InterpolationSpeed == EMassLookAtInterpolationSpeed::Custom", EditConditionHides))
	float CustomInterpolationSpeed = UE::Mass::LookAt::DefaultCustomInterpolationSpeed;

	/** Random gaze Mode */
	UPROPERTY(EditAnywhere, Category = Parameter)
	EMassLookAtGazeMode RandomGazeMode = EMassLookAtGazeMode::None;
	
	/** Random gaze yaw angle added to the look direction determined by the look at mode. */
	UPROPERTY(EditAnywhere, Category = Parameter, meta = (UIMin = 0.0, ClampMin = 0.0, UIMax = 180.0, ClampMax = 180.0))
	uint8 RandomGazeYawVariation = 0;

	/** Random gaze pitch angle added to the look direction determined by the look at mode. */
	UPROPERTY(EditAnywhere, Category = Parameter, meta = (UIMin = 0.0, ClampMin = 0.0, UIMax = 180.0, ClampMax = 180.0))
	uint8 RandomGazePitchVariation = 0;

	/** If true, allow random gaze to look at other entities too. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	bool bRandomGazeEntities = false;
};

#undef UE_API
