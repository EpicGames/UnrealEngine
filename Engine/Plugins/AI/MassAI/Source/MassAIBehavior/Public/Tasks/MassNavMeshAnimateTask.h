// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "MassNavMeshAnimateTask.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FStateTreeExecutionContext;
struct FMassMoveTargetFragment;
struct FTransformFragment;
class UMassSignalSubsystem;

USTRUCT()
struct FMassNavMeshAnimateTaskInstanceData
{
	GENERATED_BODY()

	/** Delay before the task ends. Default (0 or any negative) will run indefinitely, so it requires a transition in the state tree to stop it. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float Duration = 0.f;

	UPROPERTY()
	float Time = 0.f;
};

/**
 * Stop and animate on current navmesh location
 */
USTRUCT(meta = (DisplayName = "NavMesh Animate"))
struct FMassNavMeshAnimateTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassNavMeshAnimateTaskInstanceData;

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;

	TStateTreeExternalDataHandle<FMassMoveTargetFragment> MoveTargetHandle;
	TStateTreeExternalDataHandle<UMassSignalSubsystem> MassSignalSubsystemHandle;
	TStateTreeExternalDataHandle<FTransformFragment> TransformHandle;
};

#undef UE_API
