// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassNavigationTypes.h"
#include "MassStateTreeTypes.h"
#include "MassNavMeshFindReachablePointTask.generated.h"

struct FAgentRadiusFragment;
struct FAgentHeightFragment;
struct FStateTreeExecutionContext;
struct FTransformFragment;
namespace UE::MassBehavior
{
struct FStateTreeDependencyBuilder;
};

USTRUCT()
struct FMassNavMeshFindReachablePointTaskInstanceData
{
	GENERATED_BODY()

	/** Radius around the current entity's location to finds a random reachable location */
	UPROPERTY(EditAnywhere, Category = Parameters)
	float SearchRadius = 500.f;

	UPROPERTY(EditAnywhere, Category = Output)
	FMassTargetLocation TargetLocation;
};

/**
 * Updates TargetLocation to a random reachable location based on the agents current location on NavMesh.
 */
USTRUCT(meta = (DisplayName = "NavMesh Find Random Reachable Target"))
struct FMassNavMeshFindReachablePointTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassNavMeshFindReachablePointTaskInstanceData;

protected:
	virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}
	virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;

	TStateTreeExternalDataHandle<FTransformFragment> TransformHandle;
	TStateTreeExternalDataHandle<FAgentRadiusFragment> AgentRadiusHandle;
	TStateTreeExternalDataHandle<FAgentHeightFragment> AgentHeightHandle;
};
