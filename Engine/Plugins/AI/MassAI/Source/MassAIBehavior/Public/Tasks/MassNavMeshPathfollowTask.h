// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassMovementTypes.h"
#include "MassNavigationTypes.h"
#include "MassNavMeshNavigationFragments.h"
#include "MassStateTreeExecutionContext.h"
#include "MassStateTreeTypes.h"
#include "MassNavMeshPathfollowTask.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FTransformFragment;
struct FMassMoveTargetFragment;
struct FAgentRadiusFragment;
struct FAgentHeightFragment;
struct FMassMovementParameters;
struct FMassDesiredMovementFragment;

/** FMassNavMeshPathFollowTask movement parameters */
USTRUCT()
struct FMassNavMeshPathFollowTaskInstanceData
{ 
	GENERATED_BODY()
	
	UPROPERTY(EditAnywhere, Category = Input)
	FMassTargetLocation TargetLocation;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FMassMovementStyleRef MovementStyle;

	UPROPERTY(EditAnywhere, Category = Parameter)
	float SpeedScale = 1.f;
	
	/** Maximum width of the corridor to use. */
	UPROPERTY(EditAnywhere, Category = Parameter)
    float CorridorWidth = 600.f;

	/** Amount to offset corridor sides from navmesh borders. Used to keep movement away for navmesh borders. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float OffsetFromBoundaries = 10.f;

	/** Offset applied at path corners that touch corridor walls. Unlike OffsetFromBoundaries
	 *  (global string-pull bias), this only pushes corner points inward along their portal. */
	UPROPERTY(EditAnywhere, Category = Parameter, meta = (ClampMin = "0.0"))
	float OffsetCornersFromBoundaries = 0.f;

	/** Distance from the end of path used to confirm that the destination is reached. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float EndDistanceThreshold = 20.f;
};

/** Finds a path to TargetLocation, requests a short path, starts a move action and follow the path by updating the short path when needed. */ 
USTRUCT(meta = (DisplayName = "NavMesh Path Follow"))
struct FMassNavMeshPathFollowTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FMassNavMeshPathFollowTaskInstanceData::StaticStruct(); };

	UE_API bool RequestPath(FStateTreeExecutionContext& Context, const FMassTargetLocation& InTargetLocation) const;
	UE_API bool UpdateShortPath(FStateTreeExecutionContext& Context) const;

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;

	TStateTreeExternalDataHandle<FTransformFragment> TransformHandle;
	TStateTreeExternalDataHandle<FMassMoveTargetFragment> MoveTargetHandle;
	TStateTreeExternalDataHandle<FAgentRadiusFragment> AgentRadiusHandle;
	TStateTreeExternalDataHandle<FAgentHeightFragment> AgentHeightHandle;
	TStateTreeExternalDataHandle<FMassDesiredMovementFragment> DesiredMovementHandle;
	TStateTreeExternalDataHandle<FMassMovementParameters> MovementParamsHandle;

	// Hold a small part of a navmesh path
	TStateTreeExternalDataHandle<FMassNavMeshShortPathFragment> ShortPathHandle;

	TStateTreeExternalDataHandle<FMassNavMeshCachedPathFragment> CachedPathHandle;
};

#undef UE_API
