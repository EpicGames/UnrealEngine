// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "MassNavMeshStandTask.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FStateTreeExecutionContext;
struct FMassMoveTargetFragment;
struct FMassNavMeshShortPathFragment;
struct FMassMovementParameters;
struct FTransformFragment;
class UZoneGraphSubsystem;
class UMassSignalSubsystem;

USTRUCT()
struct FMassNavMeshStandTaskInstanceData
{
	GENERATED_BODY()

	/** Delay before the task ends. Default (0 or any negative) will run indefinitely, so it requires a transition in the state tree to stop it. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float Duration = 0.f;

	UPROPERTY()
	float Time = 0.f;
};

/**
 * Stop, and stand on current navmesh location
 */
USTRUCT(meta = (DisplayName = "NavMesh Stand"))
struct FMassNavMeshStandTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassNavMeshStandTaskInstanceData;

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;

	virtual const UStruct* GetInstanceDataType() const override
	{
		return FInstanceDataType::StaticStruct();
	}

	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;

	TStateTreeExternalDataHandle<FMassMoveTargetFragment> MoveTargetHandle;
	TStateTreeExternalDataHandle<FMassNavMeshShortPathFragment> ShortPathHandle;
	TStateTreeExternalDataHandle<FMassMovementParameters> MovementParamsHandle;
	TStateTreeExternalDataHandle<UMassSignalSubsystem> MassSignalSubsystemHandle;
	TStateTreeExternalDataHandle<FTransformFragment> TransformHandle;
};

#undef UE_API
