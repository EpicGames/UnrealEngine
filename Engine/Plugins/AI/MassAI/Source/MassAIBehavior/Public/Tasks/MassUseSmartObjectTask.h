// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassSmartObjectRequest.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassStateTreeTypes.h"
#include "SmartObjectRuntime.h"
#include "MassUseSmartObjectTask.generated.h"

struct FStateTreeExecutionContext;
struct FMassSmartObjectUserFragment;
class USmartObjectSubsystem;
class UMassSignalSubsystem;
struct FTransformFragment;
struct FMassMoveTargetFragment;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
 * Task to tell an entity to start using a claimed smart object.
 */
USTRUCT()
struct FMassUseSmartObjectTaskInstanceData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Input)
	FSmartObjectClaimHandle ClaimedSlot;
};

USTRUCT(meta = (DisplayName = "Mass Use SmartObject Task"))
struct FMassUseSmartObjectTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()
	
	using FInstanceDataType = FMassUseSmartObjectTaskInstanceData;

	MASSAIBEHAVIOR_API FMassUseSmartObjectTask();

protected:
	MASSAIBEHAVIOR_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	MASSAIBEHAVIOR_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	MASSAIBEHAVIOR_API virtual void ExitState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	MASSAIBEHAVIOR_API virtual void StateCompleted(FStateTreeExecutionContext& Context, const EStateTreeRunStatus CompletionStatus, const FStateTreeActiveStates& CompletedActiveStates) const override;
	MASSAIBEHAVIOR_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	MASSAIBEHAVIOR_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<USmartObjectSubsystem> SmartObjectSubsystemHandle;
	TStateTreeExternalDataHandle<UMassSignalSubsystem> MassSignalSubsystemHandle;
	TStateTreeExternalDataHandle<FMassSmartObjectUserFragment> SmartObjectUserHandle;
	TStateTreeExternalDataHandle<FMassMoveTargetFragment> MoveTargetHandle;
};

