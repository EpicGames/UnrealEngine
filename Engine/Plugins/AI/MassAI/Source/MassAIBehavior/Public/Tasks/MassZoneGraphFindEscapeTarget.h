// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "Tasks/MassZoneGraphPathFollowTask.h"
#include "MassZoneGraphFindEscapeTarget.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FStateTreeExecutionContext;
struct FMassZoneGraphLaneLocationFragment;
class UZoneGraphSubsystem;
class UZoneGraphAnnotationSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
 * Updates TargetLocation to a escape target based on the agents current location on ZoneGraph, and disturbance annotation.
 */
USTRUCT()
struct FMassZoneGraphFindEscapeTargetInstanceData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Output)
	FMassZoneGraphTargetLocation EscapeTargetLocation;
};

USTRUCT(meta = (DisplayName = "ZG Find Escape Target"))
struct FMassZoneGraphFindEscapeTarget : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassZoneGraphFindEscapeTargetInstanceData;
	
	UE_API FMassZoneGraphFindEscapeTarget();

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<FMassZoneGraphLaneLocationFragment> LocationHandle;
	TStateTreeExternalDataHandle<UZoneGraphSubsystem> ZoneGraphSubsystemHandle;
	TStateTreeExternalDataHandle<UZoneGraphAnnotationSubsystem> ZoneGraphAnnotationSubsystemHandle;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FZoneGraphTag DisturbanceAnnotationTag = FZoneGraphTag::None;
};

#undef UE_API
