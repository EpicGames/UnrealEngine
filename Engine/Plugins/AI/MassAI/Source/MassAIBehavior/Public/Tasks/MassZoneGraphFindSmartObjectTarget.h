// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "SmartObjectRuntime.h"
#include "MassZoneGraphPathFollowTask.h"
#include "MassZoneGraphFindSmartObjectTarget.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FMassSmartObjectUserFragment;
struct FMassZoneGraphLaneLocationFragment;
class UZoneGraphAnnotationSubsystem;
class USmartObjectSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
* Computes move target to a smart object based on current location on ZoneGraph.
*/
USTRUCT()
struct FMassZoneGraphFindSmartObjectTargetInstanceData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category = Input)
	FSmartObjectClaimHandle ClaimedSlot;

	UPROPERTY(EditAnywhere, Category = Output)
	FMassZoneGraphTargetLocation SmartObjectLocation;
};

USTRUCT(meta = (DisplayName = "ZG Find Smart Object Target"))
struct FMassZoneGraphFindSmartObjectTarget : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassZoneGraphFindSmartObjectTargetInstanceData;

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<FMassZoneGraphLaneLocationFragment> LocationHandle;
	TStateTreeExternalDataHandle<UZoneGraphAnnotationSubsystem> AnnotationSubsystemHandle;
	TStateTreeExternalDataHandle<USmartObjectSubsystem> SmartObjectSubsystemHandle;
};

#undef UE_API
