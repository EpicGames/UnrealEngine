// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassNavigationTypes.h"
#include "MassMovementTypes.h"
#include "MassCommonTypes.h"
#include "MassStateTreeTypes.h"
#include "StateTreePropertyRef.h"
#include "ZoneGraphTypes.h"
#include "MassZoneGraphPathFollowTask.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FMassStateTreeExecutionContext;
struct FMassZoneGraphLaneLocationFragment;
struct FMassMoveTargetFragment;
struct FMassZoneGraphPathRequestFragment;
struct FMassZoneGraphShortPathFragment;
struct FMassZoneGraphCachedLaneFragment;
struct FAgentRadiusFragment;
struct FMassMovementParameters;
class UZoneGraphSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

USTRUCT()
struct FMassZoneGraphTargetLocation
{
	GENERATED_BODY()

	void Reset()
	{
		LaneHandle.Reset();
		NextLaneHandle.Reset();
		NextExitLinkType = EZoneLaneLinkType::None;
		bMoveReverse = false;
		TargetDistance = 0.0f;
		EndOfPathPosition.Reset();
		AnticipationDistance.Set(50.0f);
		EndOfPathIntent = EMassMovementAction::Move;
	}

	/** Current lane handle. (Could be debug only) */
	FZoneGraphLaneHandle LaneHandle;
	
	/** If valid, the this lane will be set as current lane after the path follow is completed. */
	FZoneGraphLaneHandle NextLaneHandle;
	
	/** Target distance along current lane. */
	float TargetDistance = 0.0f;
	
	/** Optional end of path location. */
	TOptional<FVector> EndOfPathPosition;

	/** Optional end of path direction, used only if EndOfPathPosition is set. */
	TOptional<FVector> EndOfPathDirection;

	/** If start or end of path is off-lane, the distance along the lane is pushed forward/back along the lane to make smoother transition. */
	FMassInt16Real AnticipationDistance = FMassInt16Real(50.0f);

	/** True, if we're moving reverse along the lane. */
	bool bMoveReverse = false;
	
	/** Movement intent at the end of the path */
	EMassMovementAction EndOfPathIntent = EMassMovementAction::Move;
	
	/** How the next lane handle is reached relative to the current lane. */
	EZoneLaneLinkType NextExitLinkType = EZoneLaneLinkType::None;
};

/**
 * Follows a path long the current lane to a specified point.
 */
USTRUCT()
struct FMassZoneGraphPathFollowTaskInstanceData
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, Category = Input, meta=(RefType = "/Script/MassAIBehavior.MassZoneGraphTargetLocation"))
	FStateTreePropertyRef TargetLocation;

	UPROPERTY(EditAnywhere, Category = Parameter)
	FMassMovementStyleRef MovementStyle;

	UPROPERTY(EditAnywhere, Category = Parameter)
	float SpeedScale = 1.0f;
};

USTRUCT(meta = (DisplayName = "ZG Path Follow"))
struct FMassZoneGraphPathFollowTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassZoneGraphPathFollowTaskInstanceData;

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); };
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	UE_API bool RequestPath(FMassStateTreeExecutionContext& Context, const FMassZoneGraphTargetLocation& TargetLocation) const;

	TStateTreeExternalDataHandle<FMassZoneGraphLaneLocationFragment> LocationHandle;
	TStateTreeExternalDataHandle<FMassMoveTargetFragment> MoveTargetHandle;
	TStateTreeExternalDataHandle<FMassZoneGraphPathRequestFragment> PathRequestHandle;
	TStateTreeExternalDataHandle<FMassZoneGraphShortPathFragment> ShortPathHandle;
	TStateTreeExternalDataHandle<FMassZoneGraphCachedLaneFragment> CachedLaneHandle;
	TStateTreeExternalDataHandle<FAgentRadiusFragment> AgentRadiusHandle;
	TStateTreeExternalDataHandle<FMassMovementParameters> MovementParamsHandle;
	TStateTreeExternalDataHandle<UZoneGraphSubsystem> ZoneGraphSubsystemHandle;
};

#undef UE_API
