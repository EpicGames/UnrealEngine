// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassStateTreeTypes.h"
#include "MassZoneGraphStandTask.generated.h"

#define UE_API MASSAIBEHAVIOR_API

struct FStateTreeExecutionContext;
struct FMassZoneGraphLaneLocationFragment;
struct FMassMoveTargetFragment;
struct FMassZoneGraphShortPathFragment;
struct FMassZoneGraphCachedLaneFragment;
struct FMassMovementParameters;
class UZoneGraphSubsystem;
class UMassSignalSubsystem;
namespace UE::MassBehavior
{
	struct FStateTreeDependencyBuilder;
};

/**
 * Stop, and stand on current ZoneGraph location
 */

USTRUCT()
struct FMassZoneGraphStandTaskInstanceData
{
	GENERATED_BODY()

	/** Delay before the task ends. Default (0 or any negative) will run indefinitely so it requires a transition in the state tree to stop it. */
	UPROPERTY(EditAnywhere, Category = Parameter)
	float Duration = 0.0f;

	UPROPERTY()
	float Time = 0.0f;
};

USTRUCT(meta = (DisplayName = "ZG Stand"))
struct FMassZoneGraphStandTask : public FMassStateTreeTaskBase
{
	GENERATED_BODY()

	using FInstanceDataType = FMassZoneGraphStandTaskInstanceData;

protected:
	UE_API virtual bool Link(FStateTreeLinker& Linker) override;
	virtual const UStruct* GetInstanceDataType() const override { return FInstanceDataType::StaticStruct(); }
	UE_API virtual EStateTreeRunStatus EnterState(FStateTreeExecutionContext& Context, const FStateTreeTransitionResult& Transition) const override;
	UE_API virtual EStateTreeRunStatus Tick(FStateTreeExecutionContext& Context, const float DeltaTime) const override;
	UE_API virtual void GetDependencies(UE::MassBehavior::FStateTreeDependencyBuilder& Builder) const override;

	TStateTreeExternalDataHandle<FMassZoneGraphLaneLocationFragment> LocationHandle;
	TStateTreeExternalDataHandle<FMassMoveTargetFragment> MoveTargetHandle;
	TStateTreeExternalDataHandle<FMassZoneGraphShortPathFragment> ShortPathHandle;
	TStateTreeExternalDataHandle<FMassZoneGraphCachedLaneFragment> CachedLaneHandle;
	TStateTreeExternalDataHandle<FMassMovementParameters> MovementParamsHandle;
	TStateTreeExternalDataHandle<UZoneGraphSubsystem> ZoneGraphSubsystemHandle;
	TStateTreeExternalDataHandle<UMassSignalSubsystem> MassSignalSubsystemHandle;
};

#undef UE_API
