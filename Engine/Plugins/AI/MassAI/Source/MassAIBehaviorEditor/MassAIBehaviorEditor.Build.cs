// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassAIBehaviorEditor : ModuleRules
	{
		public MassAIBehaviorEditor(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicIncludePaths.AddRange(
			new string[] {
			}
			);

			PublicDependencyModuleNames.AddRange(
			new string[] {
			}
			);

			PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AIModule",
				"AppFramework",
				"Core",
				"CoreUObject",
				"Engine",
				"InputCore",
				"PropertyEditor",
				"MassAIBehavior",
				"MassCore",
				"MassEntity",
				"Slate",
				"SlateCore",
			}
			);
		}

	}
}
