// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassAIBehaviorEditorModule.h"
#include "MassLookAtPriorityDetails.h"
#include "MassLookAtPriorityInfoDetails.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"

#define LOCTEXT_NAMESPACE "MassAIBehaviorEditor"

IMPLEMENT_MODULE(FMassAIBehaviorEditorModule, MassAIBehaviorEditor)

void FMassAIBehaviorEditorModule::StartupModule()
{
	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");

	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("MassLookAtPriority"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMassLookAtPriorityDetails::MakeInstance));
	PropertyModule.RegisterCustomPropertyTypeLayout(TEXT("MassLookAtPriorityInfo"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FMassLookAtPriorityInfoDetails::MakeInstance));
}

void FMassAIBehaviorEditorModule::ShutdownModule()
{
	if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MassLookAtPriority"));
		PropertyModule.UnregisterCustomPropertyTypeLayout(TEXT("MassLookAtPriorityInfo"));
	}
}

#undef LOCTEXT_NAMESPACE
