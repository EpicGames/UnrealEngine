// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#include "Modules/ModuleInterface.h"

#define UE_API MASSAIBEHAVIOREDITOR_API

class FMassAIBehaviorEditorModule : public IModuleInterface
{
public:
	//~ Begin IModuleInterface
	UE_API virtual void StartupModule() override;
	UE_API virtual void ShutdownModule() override;
	//~ End IModuleInterface
};

#undef UE_API
