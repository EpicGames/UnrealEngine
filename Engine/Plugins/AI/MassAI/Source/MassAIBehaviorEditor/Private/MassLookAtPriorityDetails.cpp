// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtPriorityDetails.h"

#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "ISettingsModule.h"
#include "MassLookAtSettings.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Modules/ModuleManager.h"
#include "Widgets/Input/SComboButton.h"

#define LOCTEXT_NAMESPACE "MassLookAtPriorityDetails"

TSharedRef<IPropertyTypeCustomization> FMassLookAtPriorityDetails::MakeInstance()
{
	return MakeShareable(new FMassLookAtPriorityDetails);
}

void FMassLookAtPriorityDetails::CustomizeHeader(const TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Can't use GET_MEMBER_NAME_CHECKED() since property is private
	constexpr TCHAR PriorityValueName[](TEXT("Value"));
	PriorityValueProperty = StructPropertyHandle->GetChildHandle(PriorityValueName);
	checkf(PriorityValueProperty, TEXT("Unable to find property called '%s' in FMassLookAtPriority."
		" Make sure this code matches the property name and that the property is exposed to the editor."), PriorityValueName);

	// Build cache for current priorities and register for modifications
	CachePriorityInfos();
	UMassLookAtSettings::OnMassLookAtPrioritiesChanged.AddSP(this, &FMassLookAtPriorityDetails::CachePriorityInfos);

	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(200.0f)
	[
		SNew(SComboButton)
		.OnGetMenuContent(this, &FMassLookAtPriorityDetails::OnGetComboContent)
		.ContentPadding(FMargin(2.0f, 0.0f))
		.ButtonContent()
		[
			SNew(SHorizontalBox)
			// Description
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(this, &FMassLookAtPriorityDetails::GetDescription)
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
		]
	];
}

void FMassLookAtPriorityDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
}

void FMassLookAtPriorityDetails::CachePriorityInfos()
{
	PriorityInfos.Reset();
	if (const UMassLookAtSettings* Settings = GetDefault<UMassLookAtSettings>())
	{
		Settings->GetValidPriorityInfos(PriorityInfos);
	}
}

FText FMassLookAtPriorityDetails::GetDescription() const
{
	uint8 Priority = 0;
	const FPropertyAccess::Result Result = PriorityValueProperty->GetValue(Priority);
	if (Result == FPropertyAccess::Success)
	{
		FText Name = LOCTEXT("NameEmpty", "(not set)");
		for (const FMassLookAtPriorityInfo& Info : PriorityInfos)
		{
			if (Info.Priority.Get() == Priority)
			{
				Name = FText::FromName(Info.Name);
				break;
			}
		}
		return Name;
	}

	if (Result == FPropertyAccess::MultipleValues)
	{
		return LOCTEXT("MultipleSelected", "(Multiple Selected)");
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> FMassLookAtPriorityDetails::OnGetComboContent() const
{
	FMenuBuilder MenuBuilder(true, nullptr);

	FUIAction EditPrioritiesItemAction(FExecuteAction::CreateSPLambda(this, []
		{
			// Go to settings to edit priorities
			const UMassLookAtSettings* Settings = GetDefault<UMassLookAtSettings>();
			FModuleManager::LoadModuleChecked<ISettingsModule>("Settings").ShowViewer(Settings->GetContainerName(), Settings->GetCategoryName(), Settings->GetSectionName());
		}));

	MenuBuilder.AddMenuEntry(LOCTEXT("EditPriorities", "Edit Priorities..."), TAttribute<FText>(), FSlateIcon(), EditPrioritiesItemAction);
	MenuBuilder.AddMenuSeparator();

	for (const FMassLookAtPriorityInfo& Info : PriorityInfos)
	{
		const uint8 PriorityValue = Info.Priority.Get();
		FUIAction BitItemAction(FExecuteAction::CreateLambda([PriorityValueProperty = PriorityValueProperty, PriorityValue]
		{
			PriorityValueProperty->SetValue(PriorityValue);
		}));

		TSharedRef<SWidget> Widgets = SNew(SHorizontalBox)
			// Description
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(FMargin(6.0f, 2.0f))
			[
				SNew(STextBlock)
				.Text(FText::FromName(Info.Name))
			];

		MenuBuilder.AddMenuEntry(BitItemAction, Widgets);
	}

	return MenuBuilder.MakeWidget();
}

#undef LOCTEXT_NAMESPACE