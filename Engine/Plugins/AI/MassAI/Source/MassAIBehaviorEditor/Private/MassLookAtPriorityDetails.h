// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"

struct FMassLookAtPriorityInfo;
class IPropertyHandle;
class SWidget;

/**
 * Type customization for FMassLookAtPriority.
 */
class FMassLookAtPriorityDetails final : public IPropertyTypeCustomization
{
public:
	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization interface

private:

	/** @return Description for the current priority value */
	FText GetDescription() const;

	/** Builds the priority drop down */
	TSharedRef<SWidget> OnGetComboContent() const;

	/** Cache priority values from MassLookAtSetting into 'PriorityInfos' */
	void CachePriorityInfos();

	TArray<FMassLookAtPriorityInfo> PriorityInfos;
	TSharedPtr<IPropertyHandle> PriorityValueProperty;
	TSharedPtr<IPropertyHandle> StructProperty;
};