// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassLookAtPriorityInfoDetails.h"
#include "MassLookAtTypes.h"
#include "DetailWidgetRow.h"
#include "DetailLayoutBuilder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "MassLookAtPriorityInfoDetails"

FMassLookAtPriorityInfoDetails::~FMassLookAtPriorityInfoDetails()
{
}

TSharedRef<IPropertyTypeCustomization> FMassLookAtPriorityInfoDetails::MakeInstance()
{
	return MakeShareable(new FMassLookAtPriorityInfoDetails);
}

void FMassLookAtPriorityInfoDetails::CustomizeHeader(const TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	NameProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMassLookAtPriorityInfo, Name));
	PriorityProperty = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FMassLookAtPriorityInfo, Priority));

	HeaderRow
	.WholeRowContent()
	[
		SNew(SHorizontalBox)
		// Priority value
		+ SHorizontalBox::Slot()
		.FillWidth(0.1f)
		.MaxWidth(30.0f)
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			SNew(STextBlock)
			.Text(this, &FMassLookAtPriorityInfoDetails::GetPriorityDescription)
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		// Description
		+ SHorizontalBox::Slot()
		.FillWidth(1.0f)
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.0f, 2.0f))
		[
			NameProperty->CreatePropertyValueWidget()
		]
		+ SHorizontalBox::Slot()
		.Padding(FMargin(12.0f, 0.0f))
		.HAlign(HAlign_Right)
		[
			StructPropertyHandle->CreateDefaultPropertyButtonWidgets()
		]
	];
}

void FMassLookAtPriorityInfoDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	// Nothing required; pure virtual in IPropertyTypeCustomization
}

FText FMassLookAtPriorityInfoDetails::GetPriorityDescription() const
{
	if (!PriorityProperty.IsValid())
	{
		return FText::GetEmpty();
	}

	FMassLookAtPriority Value;
	bool bValueSet = false;

	TArray<void*> RawData;
	PriorityProperty->AccessRawData(RawData);
	for (void* Data : RawData)
	{
		if (Data)
		{
			FMassLookAtPriority CurValue = *static_cast<FMassLookAtPriority*>(Data);
			if (!bValueSet)
			{
				bValueSet = true;
				Value = CurValue;
			}
			else if (CurValue != Value)
			{
				// Multiple values
				return FText::GetEmpty();
			}
		}
	}

	if (Value.IsValid())
	{
		return FText::AsNumber(Value.Get());
	}
	return FText::GetEmpty();
}

#undef LOCTEXT_NAMESPACE