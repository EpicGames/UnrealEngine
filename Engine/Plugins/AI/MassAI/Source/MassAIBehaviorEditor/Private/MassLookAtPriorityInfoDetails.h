// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IPropertyTypeCustomization.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;

/**
 * Type customization for FMassLookAtPriorityInfo.
 */
class FMassLookAtPriorityInfoDetails : public IPropertyTypeCustomization
{
public:
	virtual ~FMassLookAtPriorityInfoDetails() override;

	/** Makes a new instance of this detail layout class for a specific detail view requesting it */
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	//~ Begin IPropertyTypeCustomization interface
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	//~ End IPropertyTypeCustomization interface

private:

	FText GetPriorityDescription() const;

	TSharedPtr<IPropertyHandle> PriorityProperty;
	TSharedPtr<IPropertyHandle> NameProperty;
};