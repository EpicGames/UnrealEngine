// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassAIDebug : ModuleRules
	{
		public MassAIDebug(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"InputCore",
					"MassCore",
					"MassEntity",
					"NavigationSystem",
					"StateTreeModule",
					"MassGameplayDebug",
					"MassActors",
					"MassAIBehavior",
					"MassCommon",
					"MassMovement",
					"MassNavigation",
					"MassNavMeshNavigation",
					"MassZoneGraphNavigation",
					"MassAIReplication",
					"MassSmartObjects",
					"MassSpawner",
					"MassSimulation",
					"MassRepresentation",
					"MassSignals",
					"MassLOD",
					"MassSmartObjects",
					"SmartObjectsModule",
				}
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
				PublicDependencyModuleNames.Add("MassEntityEditor");
			}

			SetupGameplayDebuggerSupport(Target);
		}
	}
}
