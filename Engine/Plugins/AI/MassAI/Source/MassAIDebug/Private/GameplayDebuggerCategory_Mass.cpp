// Copyright Epic Games, Inc. All Rights Reserved.

#include "GameplayDebuggerCategory_Mass.h"

#if WITH_GAMEPLAY_DEBUGGER && WITH_MASSGAMEPLAY_DEBUG
#include "MassGameplayDebugTypes.h"
#include "MassExecutionContext.h"
#include "GameplayDebuggerConfig.h"
#include "GameplayDebuggerCategoryReplicator.h"
#include "MassDebuggerSubsystem.h"
#include "MassActorSubsystem.h"
#include "GameFramework/PlayerController.h"
#include "MassAgentComponent.h"
#include "MassEntityManager.h"
#include "MassEntitySubsystem.h"
#include "MassNavigationFragments.h"
#include "Steering/MassSteeringFragments.h"
#include "MassLookAtFragments.h"
#include "MassLookAtSettings.h"
#include "MassLookAtSubsystem.h"
#include "MassStateTreeFragments.h"
#include "MassStateTreeExecutionContext.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassNavMeshNavigationFragments.h"
#include "MassSmartObjectFragments.h"
#include "SmartObjectSubsystem.h"
#include "MassSimulationLOD.h"
#include "CanvasItem.h"
#include "Engine/World.h"
#include "MassDebugger.h"
#include "MassLODSubsystem.h"
#include "MassMovementFragments.h"

namespace UE::Mass::Debug
{
	FMassEntityHandle GetEntityFromActor(const AActor& Actor, const UMassAgentComponent*& OutMassAgentComponent)
	{
		FMassEntityHandle EntityHandle;
		if (const UMassAgentComponent* AgentComp = Actor.FindComponentByClass<UMassAgentComponent>())
		{
			EntityHandle = AgentComp->GetEntityHandle();
			OutMassAgentComponent = AgentComp;
		}
		else if (UMassActorSubsystem* ActorSubsystem = UWorld::GetSubsystem<UMassActorSubsystem>(Actor.GetWorld()))
		{
			EntityHandle = ActorSubsystem->GetEntityHandleFromActor(&Actor);
		}
		return EntityHandle;
	};

	FMassEntityHandle GetBestEntity(const FVector ViewLocation, const FVector ViewDirection, const TConstArrayView<FMassEntityHandle> Entities
		, const TConstArrayView<FVector> Locations, const bool bLimitAngle, const FVector::FReal MaxScanDistance)
	{
		constexpr FVector::FReal MinViewDirDot = 0.707; // 45 degrees		
		const FVector::FReal MaxScanDistanceSq = MaxScanDistance * MaxScanDistance;

		checkf(Entities.Num() == Locations.Num(), TEXT("Both Entities and Locations lists are expected to be of the same size: %d vs %d"), Entities.Num(), Locations.Num());
		
		FVector::FReal BestScore = bLimitAngle ? MinViewDirDot : (-1. - KINDA_SMALL_NUMBER);
		FMassEntityHandle BestEntity;

		for (int i = 0; i < Entities.Num(); ++i)
		{
			if (Entities[i].IsSet() == false)
			{
				continue;
			}
			
			const FVector DirToEntity = (Locations[i] - ViewLocation);
			const FVector::FReal DistToEntitySq = DirToEntity.SizeSquared();
			if (DistToEntitySq > MaxScanDistanceSq)
			{
				continue;
			}

			const FVector::FReal Distance = FMath::Sqrt(DistToEntitySq);
			const FVector DirToEntityNormal = (FMath::IsNearlyZero(DistToEntitySq)) ? ViewDirection : (DirToEntity / Distance);
			const FVector::FReal ViewDot = FVector::DotProduct(ViewDirection, DirToEntityNormal);
			const FVector::FReal Score = ViewDot * 0.1 * (1. - Distance / MaxScanDistance);
			if (ViewDot > BestScore)
			{
				BestScore = ViewDot;
				BestEntity = Entities[i];
			}
		}

		return BestEntity;
	}
} // namespace UE::Mass:Debug

//----------------------------------------------------------------------//
//  FGameplayDebuggerCategory_Mass
//----------------------------------------------------------------------//
TArray<FAutoConsoleCommandWithWorld> FGameplayDebuggerCategory_Mass::ConsoleCommands;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleArchetypesBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleShapesBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleAgentFragmentsBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnPickEntityBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleEntityDetailsBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleNearEntityOverviewBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleNearEntityAvoidanceBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleNearEntityPathBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleEntityLookAtBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnCycleEntityDescriptionBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleDebugLocalEntityManagerBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnTogglePickedActorAsViewerBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnToggleDrawViewersBroadcast;
FGameplayDebuggerCategory_Mass::FOnConsoleCommandBroadcastDelegate FGameplayDebuggerCategory_Mass::OnClearActorViewersBroadcast;

FGameplayDebuggerCategory_Mass::FGameplayDebuggerCategory_Mass()
{
	CachedDebugActor = nullptr;
	bShowOnlyWithDebugActor = false;

	// @todo would be nice to have these saved in per-user settings 
	bShowArchetypes = false;
	bShowShapes = false;
	bShowAgentFragments = false;
	bPickEntity = false;
	bShowEntityDetails = false;
	bShowNearEntityOverview = true;
	bShowNearEntityAvoidance = false;
	bShowNearEntityPath = false;
	bShowEntityLookAt = false;
	bMarkEntityBeingDebugged = true;
	bDebugLocalEntityManager = false;
	bShowViewers = false;

	BindKeyPress(EKeys::A.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleArchetypes, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::S.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleShapes, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::G.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleAgentFragments, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::P.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnPickEntity, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::D.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleEntityDetails, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::O.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleNearEntityOverview, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::V.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleNearEntityAvoidance, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::N.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleNearEntityPath, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::K.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleEntityLookAt, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::E.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnCycleEntityDescription, EGameplayDebuggerInputMode::Replicated);
	ToggleDebugLocalEntityManagerInputIndex = GetNumInputHandlers();
	BindKeyPress(EKeys::L.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleDebugLocalEntityManager, EGameplayDebuggerInputMode::Local);
	BindKeyPress(EKeys::Add.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnIncreaseSearchRange, EGameplayDebuggerInputMode::Replicated);
	BindKeyPress(EKeys::Subtract.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnDecreaseSearchRange, EGameplayDebuggerInputMode::Replicated);
	TogglePickedActorAsViewerInputIndex = GetNumInputHandlers();
	BindKeyPress(EKeys::U.GetFName(), FGameplayDebuggerInputModifier::Ctrl, this, &FGameplayDebuggerCategory_Mass::OnTogglePickedActorAsViewer, EGameplayDebuggerInputMode::Replicated);
	ToggleDrawViewersInputIndex = GetNumInputHandlers();
	BindKeyPress(EKeys::U.GetFName(), FGameplayDebuggerInputModifier::Shift, this, &FGameplayDebuggerCategory_Mass::OnToggleDrawViewers, EGameplayDebuggerInputMode::Replicated);
	ClearViewersInputIndex = GetNumInputHandlers();
	BindKeyPress(EKeys::U.GetFName(), FGameplayDebuggerInputModifier::Shift + FGameplayDebuggerInputModifier::Ctrl, this, &FGameplayDebuggerCategory_Mass::OnClearActorViewers, EGameplayDebuggerInputMode::Replicated);

	if (ConsoleCommands.Num() == 0)
	{
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleArchetypes"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleArchetypesBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleShapes"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleShapesBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleAgentFragments"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleAgentFragmentsBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.PickEntity"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnPickEntityBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleEntityDetails"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleEntityDetailsBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleNearEntityOverview"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleNearEntityOverviewBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleNearEntityAvoidance"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleNearEntityAvoidanceBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleNearEntityPath"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleNearEntityPathBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleEntityLookAt"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleEntityLookAtBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.CycleEntityDescriptionVerbosity"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnCycleEntityDescriptionBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleDebugLocalEntityManager"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleDebugLocalEntityManagerBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.TogglePickedActorAsViewer"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnTogglePickedActorAsViewerBroadcast.Broadcast(InWorld); }));
		ConsoleCommands.Emplace(TEXT("gdt.mass.ToggleDrawViewers"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnToggleDrawViewersBroadcast.Broadcast(InWorld); }));	
		ConsoleCommands.Emplace(TEXT("gdt.mass.ClearActorViewers"), TEXT(""), FConsoleCommandWithWorldDelegate::CreateLambda([](UWorld* InWorld) { OnClearActorViewersBroadcast.Broadcast(InWorld); }));
	}

	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleArchetypesBroadcast, OnToggleArchetypesBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleArchetypes(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleShapesBroadcast, OnToggleShapesBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleShapes(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleAgentFragmentsBroadcast, OnToggleAgentFragmentsBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleAgentFragments(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnPickEntityBroadcast, OnPickEntityBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnPickEntity(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleEntityDetailsBroadcast, OnToggleEntityDetailsBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleEntityDetails(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleNearEntityOverviewBroadcast, OnToggleNearEntityOverviewBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleNearEntityOverview(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleNearEntityAvoidanceBroadcast, OnToggleNearEntityAvoidanceBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleNearEntityAvoidance(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleNearEntityPathBroadcast, OnToggleNearEntityPathBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleNearEntityPath(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleEntityLookAtBroadcast, OnToggleEntityLookAtBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleEntityLookAt(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnCycleEntityDescriptionBroadcast, OnCycleEntityDescriptionBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnCycleEntityDescription(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleDebugLocalEntityManagerBroadcast, OnToggleDebugLocalEntityManagerBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleDebugLocalEntityManager(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnTogglePickedActorAsViewerBroadcast, OnTogglePickedActorAsViewerBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnTogglePickedActorAsViewer(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnToggleDrawViewersBroadcast, OnToggleDrawViewersBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnToggleDrawViewers(); }})));
	ConsoleCommandHandles.Add(FDelegateHandlePair(&OnClearActorViewersBroadcast, OnClearActorViewersBroadcast.AddLambda([this](UWorld* InWorld) { if (InWorld == GetWorldFromReplicator()) { OnClearActorViewers(); }})));

	OnEntitySelectedHandle = FMassDebugger::OnEntitySelectedDelegate.AddRaw(this, &FGameplayDebuggerCategory_Mass::OnEntitySelected);
}

FGameplayDebuggerCategory_Mass::~FGameplayDebuggerCategory_Mass()
{
	FMassDebugger::OnEntitySelectedDelegate.Remove(OnEntitySelectedHandle);

	for (FDelegateHandlePair& Pair : ConsoleCommandHandles)
	{
		CA_ASSUME(Pair.Key);
		Pair.Key->Remove(Pair.Value);
	}
}

void FGameplayDebuggerCategory_Mass::SetCachedEntity(const FMassEntityHandle Entity, const FMassEntityManager& EntityManager)
{
	if (CachedEntity != Entity)
	{
		FMassDebugger::SelectEntity(EntityManager, Entity);
	}
}

void FGameplayDebuggerCategory_Mass::OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle)
{
	UWorld* World = EntityManager.GetWorld();
	if (World != GetWorldFromReplicator())
	{ 
		// ignore, this call is for a different world
		return;
	}

	AActor* BestActor = nullptr;
	if (EntityHandle.IsSet() && World)
	{
		if (const UMassActorSubsystem* ActorSubsystem = World->GetSubsystem<UMassActorSubsystem>())
		{
			BestActor = ActorSubsystem->GetActorFromHandle(EntityHandle);
		}
	}

	CachedEntity = EntityHandle;
	CachedDebugActor = BestActor;
	check(GetReplicator());
	GetReplicator()->SetDebugActor(BestActor);
}

void FGameplayDebuggerCategory_Mass::ClearCachedEntity()
{
	CachedEntity = FMassEntityHandle();
}

void FGameplayDebuggerCategory_Mass::PickEntity(const FVector& ViewLocation, const FVector& ViewDirection, const UWorld& World, FMassEntityManager& EntityManager, const bool bLimitAngle)
{
	FMassEntityHandle BestEntity;
	
	// entities indicated by UE::Mass::Debug take precedence
    if (UE::Mass::Debug::HasDebugEntities() && !UE::Mass::Debug::IsDebuggingSingleEntity())
    {
		TArray<FMassEntityHandle> Entities;
	    TArray<FVector> Locations;
	    UE::Mass::Debug::GetDebugEntitiesAndLocations(EntityManager, Entities, Locations);
	    BestEntity = UE::Mass::Debug::GetBestEntity(ViewLocation, ViewDirection, Entities, Locations, bLimitAngle, SearchRange);
    }
	else
	{
		TArray<FMassEntityHandle> Entities;
		TArray<FVector> Locations;
		FMassExecutionContext ExecutionContext(EntityManager);
		FMassEntityQuery Query(EntityManager.AsShared());
		Query.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
		Query.ForEachEntityChunk(ExecutionContext, [&Entities, &Locations](FMassExecutionContext& Context)
		{
			Entities.Append(Context.GetEntities().GetData(), Context.GetEntities().Num());
			TConstArrayView<FTransformFragment> InLocations = Context.GetFragmentView<FTransformFragment>();
			Locations.Reserve(Locations.Num() + InLocations.Num());
			for (const FTransformFragment& TransformFragment : InLocations)
			{
				Locations.Add(TransformFragment.GetTransform().GetLocation());
			}
		});

		BestEntity = UE::Mass::Debug::GetBestEntity(ViewLocation, ViewDirection, Entities, Locations, bLimitAngle, SearchRange);
	}

	SetCachedEntity(BestEntity, EntityManager);
}

TSharedRef<FGameplayDebuggerCategory> FGameplayDebuggerCategory_Mass::MakeInstance()
{
	return MakeShareable(new FGameplayDebuggerCategory_Mass());
}

void FGameplayDebuggerCategory_Mass::CollectData(APlayerController* OwnerPC, AActor* DebugActor)
{
	constexpr float ViewerCylinderMarkLength = 1500.f;
	constexpr float ViewerCylinderMarkLRadius = 30.f;

	if (bAllowLocalDataCollection)
	{
		ResetReplicatedData();
	}

	// we only want to display this if there are local/remote roles in play
	if (IsCategoryAuth() != IsCategoryLocal())
	{
		AddTextLine(FString::Printf(TEXT("Source: {yellow}%s{white}"), bDebugLocalEntityManager ? TEXT("LOCAL") : TEXT("REMOTE")));
	}

	const UWorld* World = GetDataWorld(OwnerPC, DebugActor);
	check(World);

	UMassEntitySubsystem* EntitySubsystem = UWorld::GetSubsystem<UMassEntitySubsystem>(World);
	if (EntitySubsystem == nullptr)
	{
		AddTextLine(FString::Printf(TEXT("{Red}EntitySubsystem instance is missing")));
		return;
	}
	FMassEntityManager& EntityManager = EntitySubsystem->GetMutableEntityManager();
	const UMassLookAtSubsystem* LookAtSubsystem = World->GetSubsystem<UMassLookAtSubsystem>();
	const UMassDebuggerSubsystem* Debugger = World->GetSubsystem<UMassDebuggerSubsystem>();

	const UMassAgentComponent* AgentComp = nullptr;
	
	if (bAllowLocalDataCollection)
	{
		DebugActor = CachedDebugActor.GetEvenIfUnreachable();
	}

	if (DebugActor)
	{
		const FMassEntityHandle EntityHandle = UE::Mass::Debug::GetEntityFromActor(*DebugActor, AgentComp);	
		SetCachedEntity(EntityHandle, EntityManager);
		CachedDebugActor = DebugActor;
	}
	else if (CachedDebugActor.Get())
	{
		ClearCachedEntity();
		CachedDebugActor = nullptr;
	}	
	if (CachedEntity.IsValid() == true && EntityManager.IsEntityValid(CachedEntity) == false)
	{
		ClearCachedEntity();
	}

	FVector ViewLocation = FVector::ZeroVector;
	FVector ViewDirection = FVector::ForwardVector;
	if (GetViewPoint(OwnerPC, ViewLocation, ViewDirection))
	{
		// Ideally we would have a way to register in the main picking flow but that would require more changes to
		// also support client-server picking. For now, we handle explicit mass picking requests on the authority
		if (bPickEntity)
		{
			PickEntity(ViewLocation, ViewDirection, *World, EntityManager);
			bPickEntity = false;
		}
		// if we're debugging based on UE::Mass::Debug and the range changed
		else if (CachedDebugActor == nullptr && UE::Mass::Debug::HasDebugEntities() && UE::Mass::Debug::IsDebuggingEntity(CachedEntity) == false
			&& UE::Mass::Debug::IsDebuggingSingleEntity() == false)
		{
			// using bLimitAngle = false to not limit the selection to only the things in from of the player
			PickEntity(ViewLocation, ViewDirection, *World, EntityManager, /*bLimitAngle=*/false);
		}
	}

	AddTextLine(FString::Printf(TEXT("{Green}Entities count active{grey}/all: {white}%d{grey}/%d"), EntityManager.DebugGetEntityCount(), EntityManager.DebugGetEntityCount()));
	AddTextLine(FString::Printf(TEXT("{Green}Registered Archetypes count: {white}%d {green}data ver: {white}%d"), EntityManager.DebugGetArchetypesCount(), EntityManager.GetArchetypeDataVersion()));

	AddTextLine(FString::Printf(TEXT("{Green}Search range: {White}%.0f"), SearchRange));

	const FTransformFragment* TransformFragment = nullptr;
	if (CachedEntity.IsValid())
	{
		AddTextLine(FString::Printf(TEXT("{Green}Entity: {White}%s"), *CachedEntity.DebugGetDescription()));
		TransformFragment = EntityManager.GetFragmentDataPtr<FTransformFragment>(CachedEntity);
		if (TransformFragment)
		{
			AddTextLine(FString::Printf(TEXT("{Green}Distance: {White}%.0f"), FVector::Distance(TransformFragment->GetTransform().GetLocation(), ViewLocation)));
		}
	}

	if (UE::Mass::Debug::HasDebugEntities())
	{
		int32 RangeBegin, RangeEnd;
		UE::Mass::Debug::GetDebugEntitiesRange(RangeBegin, RangeEnd);
		// not printing single-entity range, since in that case the CachedEntity is already set to the appropriate entity
		if (RangeBegin != RangeEnd)
		{
			AddTextLine(FString::Printf(TEXT("{Green}Debugged entity range: {orange}%d-%d"), RangeBegin, RangeEnd));
		}
	}

	if (bShowArchetypes)
	{
		FStringOutputDevice Ar;
		Ar.SetAutoEmitLineTerminator(true);
		EntityManager.DebugPrintArchetypes(Ar, /*bIncludeEmpty*/false);

		AddTextLine(Ar);
	}

	if (bShowViewers)
	{
		if (UMassLODSubsystem* LODSubsystem = World->GetSubsystem<UMassLODSubsystem>())
		{
			for (const FViewerInfo& Viewer : LODSubsystem->GetViewers())
			{
				AddShape(FGameplayDebuggerShape::MakeCylinder(Viewer.Location, ViewerCylinderMarkLRadius, ViewerCylinderMarkLength, FColor::Blue));
			}
		}
	}

	if (CachedEntity.IsSet() && bMarkEntityBeingDebugged && TransformFragment)
	{
		const FVector Location = TransformFragment->GetTransform().GetLocation();
		AddShape(FGameplayDebuggerShape::MakeBox(Location, FVector(8,8,500), FColor::Purple,  FString::Printf(TEXT("[%s]"), *CachedEntity.DebugGetDescription())));
		AddShape(FGameplayDebuggerShape::MakePoint(Location, 10, FColor::Purple));
	}

	if (CachedEntity.IsSet() && Debugger)
	{
		AddTextLine(Debugger->GetSelectedEntityInfo());
	}

	//@todo could shave off some perf cost if UMassDebuggerSubsystem used FGameplayDebuggerShape directly
	if (bShowShapes && Debugger)
	{
		// EMassEntityDebugShape::Box
		Debugger->ForEachShape(EMassEntityDebugShape::Box, [this](const UMassDebuggerSubsystem::FShapeDesc& Desc)
			{
				AddShape(FGameplayDebuggerShape::MakeBox(Desc.Location, FVector(Desc.Size), FColor::Blue));
			});

		// EMassEntityDebugShape::Cone
		// note that we're modifying the Size here because MakeCone is using the third param as Cone's "height", while all mass debugger shapes are created with agent radius
		// FGameplayDebuggerShape::Draw is using 0.25 rad for cone angle, so that's what we'll use here
		static const float Tan025Rad = FMath::Tan(0.25f);
		Debugger->ForEachShape(EMassEntityDebugShape::Cone, [this](const UMassDebuggerSubsystem::FShapeDesc& Desc)
			{
				AddShape(FGameplayDebuggerShape::MakeCone(Desc.Location, FVector::UpVector, Desc.Size / Tan025Rad, FColor::Orange));
			});

		// EMassEntityDebugShape::Cylinder
		Debugger->ForEachShape(EMassEntityDebugShape::Cylinder, [this](const UMassDebuggerSubsystem::FShapeDesc& Desc)
			{
				AddShape(FGameplayDebuggerShape::MakeCylinder(Desc.Location, Desc.Size, Desc.Size * 2, FColor::Yellow));
			});

		// EMassEntityDebugShape::Capsule
		Debugger->ForEachShape(EMassEntityDebugShape::Capsule, [this](const UMassDebuggerSubsystem::FShapeDesc& Desc)
			{
				AddShape(FGameplayDebuggerShape::MakeCapsule(Desc.Location, Desc.Size, Desc.Size * 2, FColor::Green));
			});
	}

	if (bShowAgentFragments)
	{
		if (CachedEntity.IsSet())
		{
			// CachedEntity can become invalid if the entity "dies" or in editor mode when related actor gets moved 
			// (which causes the MassAgentComponent destruction and recreation).
			if (EntityManager.IsEntityActive(CachedEntity))
			{
				AddTextLine(FString::Printf(TEXT("{Green}Type: {White}%s"), (AgentComp == nullptr) ? TEXT("N/A") : AgentComp->IsPuppet() ? TEXT("PUPPET") : TEXT("AGENT")));

				if (bShowEntityDetails)
				{
					FStringOutputDevice FragmentsDesc;
					FragmentsDesc.SetAutoEmitLineTerminator(true);
					const TCHAR* PrefixToRemove = TEXT("DataFragment_");
					FMassDebugger::OutputEntityDescription(FragmentsDesc, EntityManager, CachedEntity, PrefixToRemove);
					AddTextLine(FString::Printf(TEXT("{Green}Fragments:\n{White}%s"), *FragmentsDesc));
				}
				else
				{
					const FMassArchetypeHandle Archetype = EntityManager.GetArchetypeForEntity(CachedEntity);
					const FMassArchetypeCompositionDescriptor& Composition = EntityManager.GetArchetypeComposition(Archetype);
					
					auto DescriptionBuilder = [](const TArray<FName>& ItemNames) -> FString {
						constexpr int ColumnsCount = 2;
						FString Description;
						int ColumnIndex = 0;
						for (const FName Name : ItemNames)
						{
							if (++ColumnIndex >= ColumnsCount)
							{
								ColumnIndex = 0;
								Description += TEXT("\n");
							}
							Description += FString::Printf(TEXT("%s,\t"), *Name.ToString());
						}
						return Description;
					};

					TArray<FName> ItemNames;
					Composition.Get<FMassTagBitSet>().DebugGetIndividualNames(ItemNames);
					AddTextLine(FString::Printf(TEXT("{Green}Tags:{White}%s"), *DescriptionBuilder(ItemNames)));
					
					ItemNames.Reset();
					Composition.Get<FMassFragmentBitSet>().DebugGetIndividualNames(ItemNames);
					AddTextLine(FString::Printf(TEXT("{Green}Fragments:{White}%s"), *DescriptionBuilder(ItemNames)));
					
					ItemNames.Reset();
					Composition.Get<FMassChunkFragmentBitSet>().DebugGetIndividualNames(ItemNames);
					AddTextLine(FString::Printf(TEXT("{Green}Chunk Fragments:{White}%s"), *DescriptionBuilder(ItemNames)));

					ItemNames.Reset();
					Composition.Get<FMassSharedFragmentBitSet>().DebugGetIndividualNames(ItemNames);
					AddTextLine(FString::Printf(TEXT("{Green}Shared Fragments:{White}%s"), *DescriptionBuilder(ItemNames)));

					ItemNames.Reset();
					Composition.Get<FMassConstSharedFragmentBitSet>().DebugGetIndividualNames(ItemNames);
					AddTextLine(FString::Printf(TEXT("{Green}Const Shared Fragments:{White}%s"), *DescriptionBuilder(ItemNames)));
				}

				if (TransformFragment)
				{
					constexpr float CapsuleRadius = 50.f;
					AddShape(FGameplayDebuggerShape::MakeCapsule(TransformFragment->GetTransform().GetLocation() + 2.f * CapsuleRadius * FVector::UpVector, CapsuleRadius, CapsuleRadius * 2.f, FColor::Orange));
				}
			}
			else
			{
				CachedEntity.Reset();
			}
		}
		else
		{
			AddTextLine(FString::Printf(TEXT("{Green}Entity: {Red}INACTIVE")));
		}
	}

	NearEntityDescriptions.Reset();
	if (bShowNearEntityOverview && OwnerPC)
	{
		FMassEntityQuery EntityQuery(EntityManager.AsShared());
		EntityQuery.AddRequirement<FMassStateTreeInstanceFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddConstSharedRequirement<FMassStateTreeSharedFragment>(EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
		EntityQuery.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassSteeringFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassStandingSteeringFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassGhostLocationFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassForceFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassLookAtFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassLookAtTargetFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassSimulationLODFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassZoneGraphShortPathFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassNavMeshShortPathFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassNavMeshCachedPathFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddRequirement<FMassSmartObjectUserFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
		EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::Optional);

		const double CurrentTime = World->GetTimeSeconds();
		
		UMassStateTreeSubsystem* MassStateTreeSubsystem = World->GetSubsystem<UMassStateTreeSubsystem>();
		USmartObjectSubsystem* SmartObjectSubsystem = World->GetSubsystem<USmartObjectSubsystem>();
		
		if (MassStateTreeSubsystem && SmartObjectSubsystem)
		{
			FMassExecutionContext Context(EntityManager, 0.0f);
		
			EntityQuery.ForEachEntityChunk(Context, [this, MassStateTreeSubsystem, SmartObjectSubsystem, LookAtSubsystem, ViewLocation, ViewDirection, CurrentTime](FMassExecutionContext& Context)
			{
				FMassEntityManager& EntityManager = Context.GetEntityManagerChecked();

				const int32 NumEntities = Context.GetNumEntities();
				const TConstArrayView<FMassStateTreeInstanceFragment> StateTreeInstanceList = Context.GetFragmentView<FMassStateTreeInstanceFragment>();
				const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
				const TConstArrayView<FAgentRadiusFragment> RadiusList = Context.GetFragmentView<FAgentRadiusFragment>();
				const TConstArrayView<FMassSteeringFragment> SteeringList = Context.GetFragmentView<FMassSteeringFragment>();
				const TConstArrayView<FMassStandingSteeringFragment> StandingSteeringList = Context.GetFragmentView<FMassStandingSteeringFragment>();
				const TConstArrayView<FMassGhostLocationFragment> GhostList = Context.GetFragmentView<FMassGhostLocationFragment>();
				const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();
				const TConstArrayView<FMassDesiredMovementFragment> DesiredMovementList = Context.GetFragmentView<FMassDesiredMovementFragment>();
				const TConstArrayView<FMassForceFragment> ForceList = Context.GetFragmentView<FMassForceFragment>();
				const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();
				const TConstArrayView<FMassLookAtFragment> LookAtList = Context.GetFragmentView<FMassLookAtFragment>();
				const TConstArrayView<FMassLookAtTargetFragment> LookAtTargetList = Context.GetFragmentView<FMassLookAtTargetFragment>();
				const TConstArrayView<FMassSimulationLODFragment> SimLODList = Context.GetFragmentView<FMassSimulationLODFragment>();
				const TConstArrayView<FMassZoneGraphShortPathFragment> ZoneGraphShortPathList = Context.GetFragmentView<FMassZoneGraphShortPathFragment>();
				const TConstArrayView<FMassNavMeshShortPathFragment> NavMeshShortPathList = Context.GetFragmentView<FMassNavMeshShortPathFragment>();
				const TConstArrayView<FMassNavMeshCachedPathFragment> NavMeshPathList = Context.GetFragmentView<FMassNavMeshCachedPathFragment>();
				const TConstArrayView<FMassSmartObjectUserFragment> SOUserList = Context.GetFragmentView<FMassSmartObjectUserFragment>();

				const bool bHasForce = (ForceList.Num() > 0);
				const bool bHasGhostLocation = (GhostList.Num() > 0);
				const bool bHasLOD = (SimLODList.Num() > 0);
				const bool bHasLookAt = (LookAtList.Num() > 0);
				const bool bHasMoveTarget = (MoveTargetList.Num() > 0);
				const bool bHasRadius = (RadiusList.Num() > 0);
				const bool bHasSOUser = (SOUserList.Num() > 0);
				const bool bHasStandingSteering = (StandingSteeringList.Num() > 0);
				const bool bHasStateTree = (StateTreeInstanceList.Num() > 0);
				const bool bHasSteering = (SteeringList.Num() > 0);
				const bool bHasVelocity = (VelocityList.Num() > 0);
				const bool bHasDesiredMovement = (DesiredMovementList.Num() > 0);
				const bool bHasNavMeshShortPaths = (NavMeshShortPathList.Num() > 0);
				const bool bHasNavMeshPaths = (NavMeshPathList.Num() > 0);

				const UGameplayDebuggerUserSettings* Settings = GetDefault<UGameplayDebuggerUserSettings>();
				const FVector::FReal MaxViewDistance = Settings->MaxViewDistance;
				const FVector::FReal MinViewDirDot = FMath::Cos(FMath::DegreesToRadians(Settings->MaxViewAngle));

				for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
				{
					FMassEntityHandle Entity = Context.GetEntity(EntityIndex);
					const FTransformFragment& Transform = TransformList[EntityIndex];
					const FVector EntityLocation = Transform.GetTransform().GetLocation();
					
					// Cull entities
					const FVector DirToEntity = EntityLocation - ViewLocation;
					const FVector::FReal DistanceToEntitySq = DirToEntity.SquaredLength();
					if (DistanceToEntitySq > FMath::Square(MaxViewDistance))
					{
						continue;
					}
					const FVector::FReal ViewDot = FVector::DotProduct(DirToEntity.GetSafeNormal(), ViewDirection);
					if (ViewDot < MinViewDirDot)
					{
						continue;
					}

					// Draw entity position and orientation.
					const FVector ZBaseOffset(0, 0, 20);
					const FVector ZDeltaOffset(0, 0, 1);
					FVector BasePos = EntityLocation + ZBaseOffset;
					const FVector EntityForward = Transform.GetTransform().GetRotation().GetForwardVector();
					constexpr float DebugShapeRadius = 10.f;

					if (bHasRadius)
					{
						AddShape(FGameplayDebuggerShape::MakeCircle(BasePos, FVector::UpVector, RadiusList[EntityIndex].Radius, FColor::White));
						AddShape(FGameplayDebuggerShape::MakeSegment(BasePos, BasePos + EntityForward * RadiusList[EntityIndex].Radius, FColor::White));
					}
					else
					{
						AddShape(FGameplayDebuggerShape::MakeCircle(BasePos, FVector::UpVector, DebugShapeRadius, FColor::Red));
						AddShape(FGameplayDebuggerShape::MakeSegment(BasePos, BasePos + EntityForward * DebugShapeRadius, FColor::Red));
					}

					if (bHasVelocity)
					{
						// Velocity target
						BasePos += ZDeltaOffset;
						AddShape(FGameplayDebuggerShape::MakeArrow(BasePos, BasePos + VelocityList[EntityIndex].Value, 10.0f, 2.0f, FColor::Yellow));
					}

					if (bHasSteering)
					{
						// Steering target
						BasePos += ZDeltaOffset;
						AddShape(FGameplayDebuggerShape::MakeArrow(BasePos, BasePos + SteeringList[EntityIndex].DesiredVelocity, 10.0f, 1.0f, FColorList::Pink));
					}

					// Look at
					if (bShowEntityLookAt && bHasLookAt)
					{
						constexpr FVector::FReal LookArrowLength = 100.;
						constexpr float TargetArrowHeadSize = 10.f;
						constexpr float TargetArrowThickness = 1.f;
						constexpr float LookArrowHeadSize = 20.f;
						constexpr float LookArrowThickness = 2.f;

						FVector LookAtBasePos = EntityLocation;

						// Apply the offset for the entity looking at something
						if (LookAtTargetList.Num())
						{
							LookAtBasePos += LookAtTargetList[EntityIndex].Offset;
						}
						else
						{
							const FVector TargetOffset = GetDefault<UMassLookAtSettings>()->GetDefaultTargetLocationOffset();
							LookAtBasePos += TargetOffset.IsZero() ? FVector(0, 0, 160) : TargetOffset;
						}

						const FMassLookAtFragment& LookAt = LookAtList[EntityIndex];
						const FVector WorldLookDirection = Transform.GetTransform().TransformVector(LookAt.Direction);
						bool bLookArrowDrawn = false;
						if (LookAt.LookAtMode == EMassLookAtMode::LookAtEntity && EntityManager.IsEntityValid(LookAt.TrackedEntity))
						{
							// Target location is expected to include its own offset
							FVector TargetPosition = LookAt.MainTargetLocation;
							AddShape(FGameplayDebuggerShape::MakeCircle(TargetPosition, FVector::UpVector, bHasRadius ? RadiusList[EntityIndex].Radius : DebugShapeRadius, FColor::Red));
							AddShape(FGameplayDebuggerShape::MakeArrow(LookAtBasePos, TargetPosition, TargetArrowHeadSize, TargetArrowThickness, FColor::Red));

							const FVector::FReal TargetDistance = FMath::Max(LookArrowLength, FVector::DotProduct(WorldLookDirection, TargetPosition - LookAtBasePos));
							AddShape(FGameplayDebuggerShape::MakeArrow(LookAtBasePos, LookAtBasePos + WorldLookDirection * TargetDistance, LookArrowHeadSize, LookArrowThickness, FColorList::LightGrey));
							bLookArrowDrawn = true;
						}

						if (LookAt.bRandomGazeEntities && EntityManager.IsEntityValid(LookAt.GazeTrackedEntity))
						{
							// Target location is expected to include its own offset
							FVector TargetPosition = LookAt.GazeTargetLocation;
							AddShape(FGameplayDebuggerShape::MakeCircle(TargetPosition, FVector::UpVector, bHasRadius ? RadiusList[EntityIndex].Radius : DebugShapeRadius, FColorList::Goldenrod));
							AddShape(FGameplayDebuggerShape::MakeArrow(LookAtBasePos, TargetPosition, TargetArrowHeadSize, TargetArrowThickness, FColorList::Goldenrod));
						}

						if (!bLookArrowDrawn)
						{
							AddShape(FGameplayDebuggerShape::MakeArrow(LookAtBasePos, LookAtBasePos + WorldLookDirection * LookArrowLength, LookArrowHeadSize, LookArrowThickness, FColor::Turquoise));
						}
					}

					// SmartObject
					if (bHasSOUser)
					{
						const FMassSmartObjectUserFragment& SOUser = SOUserList[EntityIndex];
						if (SOUser.InteractionHandle.IsValid())
						{
							const FTransform SlotTransform = SmartObjectSubsystem->GetSlotTransform(SOUser.InteractionHandle).Get(FTransform::Identity);
							const FVector SlotLocation = SlotTransform.GetLocation();
							AddShape(FGameplayDebuggerShape::MakeSegment(EntityLocation + ZBaseOffset, SlotLocation + ZBaseOffset, 3.0f, FColorList::Orange));
						}
					}

					// Desired velocity
					if (bHasDesiredMovement)
					{
						BasePos += ZDeltaOffset;
						const FVector& DesiredVelocity = DesiredMovementList[EntityIndex].DesiredVelocity;
						AddShape(FGameplayDebuggerShape::MakeArrow(BasePos, BasePos + DesiredVelocity, 10.0f, 2.0f, FColor::Blue));
					}
					
					const bool bIsLODOff = Context.DoesArchetypeHaveTag<FMassOffLODTag>();
					if (bIsLODOff)
					{
						// Draw a slightly bigger circle than the entity radius.
						AddShape(FGameplayDebuggerShape::MakeCircle(BasePos+FVector(0,0,1), FVector::UpVector,
							bHasRadius ? RadiusList[EntityIndex].Radius + 5 : DebugShapeRadius + 5, FColor::Purple)); 
					}					
					
					// Path
					if (bShowNearEntityPath && bHasMoveTarget && bHasRadius)
					{
						// Move target
						const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIndex];
						const FVector MoveBasePos = MoveTarget.Center + ZBaseOffset + ZDeltaOffset;
						AddShape(FGameplayDebuggerShape::MakeCircle(MoveBasePos, FVector::UpVector, 5.0f, FColorList::MediumVioletRed));
						AddShape(FGameplayDebuggerShape::MakeArrow(MoveBasePos,
							MoveBasePos + MoveTarget.Forward * RadiusList[EntityIndex].Radius, 10.0f, 1.0f, FColorList::MediumVioletRed));
					
						if (!ZoneGraphShortPathList.IsEmpty())
						{
							const FMassZoneGraphShortPathFragment& ShortPath = ZoneGraphShortPathList[EntityIndex];
							
							for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints - 1; PointIndex++)
							{
								const FMassZoneGraphPathPoint& CurrPoint = ShortPath.Points[PointIndex];
								const FMassZoneGraphPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];
								AddShape(FGameplayDebuggerShape::MakeSegment(CurrPoint.Position + ZBaseOffset, NextPoint.Position + ZBaseOffset, /*Thickness*/3.f, FColorList::Grey));
							}
					
							for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints; PointIndex++)
							{
								const FMassZoneGraphPathPoint& CurrPoint = ShortPath.Points[PointIndex];
								const FVector CurrBase = CurrPoint.Position + ZBaseOffset;
								// Lane tangents
								AddShape(FGameplayDebuggerShape::MakeSegment(CurrBase, CurrBase + CurrPoint.Tangent.GetVector() * 50.f, /*Thickness*/1.f, FColorList::LightGrey));
							}
						}

						if (bHasNavMeshShortPaths && Entity == CachedEntity)
						{
							const FMassNavMeshShortPathFragment& ShortPath = NavMeshShortPathList[EntityIndex];

							FColor CorridorColor = FColorList::Yellow;
							if (bHasNavMeshPaths)
							{
								const FMassNavMeshCachedPathFragment& Path = NavMeshPathList[EntityIndex];
								CorridorColor = Path.PathSource == EMassNavigationPathSource::NavMesh ? FColorList::Cyan : (Path.PathSource == EMassNavigationPathSource::Spline) ? FColorList::Blue : FColorList::Black;
							}

							for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints - 1; PointIndex++)
							{
								const FMassNavMeshPathPoint& CurrPoint = ShortPath.Points[PointIndex];
								const FMassNavMeshPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];
								AddShape(FGameplayDebuggerShape::MakeSegment(CurrPoint.Position + ZBaseOffset, NextPoint.Position + ZBaseOffset, /*Thickness*/3.f, FColorList::GreenYellow));
								AddShape(FGameplayDebuggerShape::MakeSegment(CurrPoint.Left + ZBaseOffset, NextPoint.Left + ZBaseOffset, /*Thickness*/2.f, CorridorColor));
								AddShape(FGameplayDebuggerShape::MakeSegment(CurrPoint.Right + ZBaseOffset, NextPoint.Right + ZBaseOffset, /*Thickness*/2.f, CorridorColor));
							}
					
							for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints; PointIndex++)
							{
								const FMassNavMeshPathPoint& CurrPoint = ShortPath.Points[PointIndex];
								const FVector CurrBase = CurrPoint.Position + ZBaseOffset;
								// Path tangents
								AddShape(FGameplayDebuggerShape::MakeSegment(CurrBase, CurrBase + CurrPoint.Tangent.GetVector() * 50.f, /*Thickness*/1.f, FColorList::LightGrey));
							}
						}
					}

					if (bShowNearEntityAvoidance && bHasMoveTarget && bHasRadius && bHasGhostLocation)
					{
						const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIndex];
						const FAgentRadiusFragment& Radius = RadiusList[EntityIndex];
						const FMassGhostLocationFragment& Ghost = GhostList[EntityIndex];

						// Standing avoidance.
						if (Ghost.IsValid(MoveTarget.GetCurrentActionID()))
						{
							FVector GhostBasePos = Ghost.Location + ZBaseOffset;
							AddShape(FGameplayDebuggerShape::MakeCircle(GhostBasePos, FVector::UpVector, Radius.Radius, FColorList::LightGrey));
							GhostBasePos += FVector(0,0,5);
							AddShape(FGameplayDebuggerShape::MakeArrow(GhostBasePos, GhostBasePos + Ghost.Velocity, 10.0f, 2.0f, FColorList::LightGrey));

							if (bHasStandingSteering)
							{
								const FVector GhostTargetBasePos = StandingSteeringList[EntityIndex].TargetLocation + FVector(0.0f, 0.0f, 25.0f);
								AddShape(FGameplayDebuggerShape::MakeCircle(GhostTargetBasePos, FVector::UpVector, Radius.Radius * 0.75f, FColorList::Orange));
							}
						}
					}
					
					// Status
					if (EntityDescriptionVerbosity >= EEntityDescriptionVerbosity::Minimal
						&& DistanceToEntitySq < FMath::Square(MaxViewDistance * 0.5f))
					{
						FString Status;

						// Entity name
						Status += TEXT("{orange}");
						Status += Entity.DebugGetDescription();

						if (EntityDescriptionVerbosity >= EEntityDescriptionVerbosity::Full)
						{
							// LOD
							if (bHasLOD)
							{
								Status += TEXT(" {white}LOD ");
								switch (SimLODList[EntityIndex].LOD)
								{
								case EMassLOD::High:
									Status += TEXT("High");
									break;
								case EMassLOD::Medium:
									Status += TEXT("Med");
									break;
								case EMassLOD::Low:
									Status += TEXT("Low");
									break;
								case EMassLOD::Off:
									Status += TEXT("Off");
									break;
								default:
									Status += TEXT("?");
									break;
								}
							}
							Status += TEXT("\n");

							// StateTree
							if (bHasStateTree)
							{
								// Current StateTree task
								// Optional shared fragment is expected to be present for entities with a StateTree instance
								const FMassStateTreeSharedFragment& SharedStateTree = Context.GetConstSharedFragment<FMassStateTreeSharedFragment>();
								if (const UStateTree* StateTree = SharedStateTree.StateTree)
								{
									if (FStateTreeInstanceData* InstanceData = MassStateTreeSubsystem->GetInstanceData(StateTreeInstanceList[EntityIndex].InstanceHandle))
									{
										FStateTreeReadOnlyExecutionContext StateTreeContext(MassStateTreeSubsystem, StateTree, *InstanceData);
										Status += StateTreeContext.GetActiveStateName();
										Status += FString::Printf(TEXT("  {yellow}%d{white}\n"), StateTreeContext.GetStateChangeCount());
									}
									else
									{
										Status += TEXT("{red}<No StateTree instance>{white}\n");
									}
								}
							}

							// Movement info
							if (bHasMoveTarget)
							{
								const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIndex];
								if (bHasVelocity && bHasForce)
								{
									Status += FString::Printf(TEXT("{yellow}%s/%03d {lightgrey}Speed:{white}%.1f {lightgrey}Force:{white}%.1f\n"),
										*UEnum::GetDisplayValueAsText(MoveTarget.GetCurrentAction()).ToString(), MoveTarget.GetCurrentActionID(), VelocityList[EntityIndex].Value.Length(), ForceList[EntityIndex].Value.Length());
								}

								Status += FString::Printf(TEXT("{pink}-> %s {white}Dist: %.1f\n"),
									*UEnum::GetDisplayValueAsText(MoveTarget.IntentAtGoal).ToString(), MoveTarget.DistanceToGoal);
							}

							if (bShowNearEntityPath && bHasNavMeshShortPaths)
							{
								// Display more movement info
								const FMassNavMeshShortPathFragment ShortPath = NavMeshShortPathList[EntityIndex];
								if (ShortPath.bInitialized)
								{
									Status += FString::Printf(TEXT("ShortPath: %i pts, progress: %0.f\n%s (%s) %s\n"),
										ShortPath.NumPoints,
										ShortPath.MoveTargetProgressDistance,
										ShortPath.bDone ? TEXT("{green}done{yellow}") : TEXT("{yellow}in progress"),
										ShortPath.bPartialResult ? TEXT("{yellow}partial") : TEXT("{yellow}final"),
										ShortPath.bDone && !ShortPath.bPartialResult ? TEXT("{green}PATH COMPLETED") : TEXT(""));
								}
							}

							// LookAt
							if (bShowEntityLookAt && bHasLookAt)
							{
								const FMassLookAtFragment& LookAt = LookAtList[EntityIndex];
								const double RemainingTime = LookAt.GazeDuration - (CurrentTime - LookAt.GazeStartTime);
								Status += FString::Printf(TEXT("{turquoise}%s/%s {lightgrey}%.1f\n"),
									*UEnum::GetDisplayValueAsText(LookAt.LookAtMode).ToString(), *UEnum::GetDisplayValueAsText(LookAt.RandomGazeMode).ToString(), RemainingTime);

								if (LookAtSubsystem != nullptr)
								{
									Status += FString::Printf(TEXT("{lightgrey}%s\n"), *LookAtSubsystem->DebugGetRequestsString(Entity));
								}
							}
						}

						if (!Status.IsEmpty())
						{
							BasePos += FVector(0,0,50);
							constexpr FVector::FReal ViewWeight = 0.6f; // Higher the number the more the view angle affects the score.
							const FVector::FReal ViewScale = 1. - (ViewDot / MinViewDirDot); // Zero at center of screen
							NearEntityDescriptions.Emplace(static_cast<float>(DistanceToEntitySq * ((1. - ViewWeight) + ViewScale * ViewWeight)), BasePos, Status);
						}
					}
				}
			});
		}

		if (bShowNearEntityAvoidance)
		{
			FMassEntityQuery EntityColliderQuery(EntityManager.AsShared());
			EntityColliderQuery.AddRequirement<FMassAvoidanceColliderFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
			EntityColliderQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
			EntityColliderQuery.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly);
			FMassExecutionContext Context(EntityManager, 0.f);
			EntityColliderQuery.ForEachEntityChunk(Context, [this, ViewLocation, ViewDirection](const FMassExecutionContext& Context)
			{
				const int32 NumEntities = Context.GetNumEntities();
				const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
				const TConstArrayView<FMassAvoidanceColliderFragment> CollidersList = Context.GetFragmentView<FMassAvoidanceColliderFragment>();
				const TConstArrayView<FAgentRadiusFragment> RadiiList = Context.GetFragmentView<FAgentRadiusFragment>();

				const bool bHasCollider = (CollidersList.Num() > 0);
					
				for (int32 EntityIndex = 0; EntityIndex < NumEntities; ++EntityIndex)
				{
					const FTransformFragment& Transform = TransformList[EntityIndex];
					const FVector EntityLocation = Transform.GetTransform().GetLocation();
					const FVector EntityForward = Transform.GetTransform().GetRotation().GetForwardVector();
					
					FVector BasePos = EntityLocation + FVector(0.0f ,0.0f ,25.0f );

					// Cull entities
					if (!IsLocationInViewCone(ViewLocation, ViewDirection, EntityLocation))
					{
						continue;
					}
					
					// Display colliders
					if (bHasCollider)
					{
						const FMassAvoidanceColliderFragment& Collider = CollidersList[EntityIndex];
						if (Collider.Type == EMassColliderType::Circle)
						{
							AddShape(FGameplayDebuggerShape::MakeCircle(BasePos, FVector::UpVector, Collider.GetCircleCollider().Radius, FColor::Blue));
						}
						else if (Collider.Type == EMassColliderType::Pill)
						{
							const FMassPillCollider& Pill = Collider.GetPillCollider();
							AddShape(FGameplayDebuggerShape::MakeCircle(BasePos + Pill.HalfLength * EntityForward, FVector::UpVector, Pill.Radius, FColor::Blue));
							AddShape(FGameplayDebuggerShape::MakeCircle(BasePos - Pill.HalfLength * EntityForward, FVector::UpVector, Pill.Radius, FColor::Blue));
						}
					}
					else
					{
						// Fallback on radius
						const FAgentRadiusFragment& RadiusFragment = RadiiList[EntityIndex];
						AddShape(FGameplayDebuggerShape::MakeCircle(BasePos + FVector(0,0,2), FVector::UpVector, RadiusFragment.Radius, FColor::Orange));
					}
				}
			});
		}
		
		// Cap labels to closest ones.
		NearEntityDescriptions.Sort([](const FEntityDescription& LHS, const FEntityDescription& RHS){ return LHS.Score < RHS.Score; });
		constexpr int32 MaxLabels = 15;
		if (NearEntityDescriptions.Num() > MaxLabels)
		{
			NearEntityDescriptions.RemoveAt(MaxLabels, NearEntityDescriptions.Num() - MaxLabels);
		}
	}

	if (const UMassLookAtSubsystem* MassLookAtSubsystem = World->GetSubsystem<UMassLookAtSubsystem>())
	{
		AddTextLine(FString::Printf(TEXT("%d LookAt targets"), MassLookAtSubsystem->DebugGetRegisteredTargetCount()));
	}

	if (const UMassActorSubsystem* ActorSubsystem = World->GetSubsystem<UMassActorSubsystem>())
	{
		AddTextLine(FString::Printf(TEXT("%d actor/entity pairs"), ActorSubsystem->DebugGetRegisteredActorCount()));
	}
}

void FGameplayDebuggerCategory_Mass::DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext)
{
	CanvasContext.Printf(TEXT("\n[{yellow}%s{white}] %s Archetypes"), *GetInputHandlerDescription(0), bShowArchetypes ? TEXT("Hide") : TEXT("Show"));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s Shapes"), *GetInputHandlerDescription(1), bShowShapes ? TEXT("Hide") : TEXT("Show"));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s Agent Fragments"), *GetInputHandlerDescription(2), bShowAgentFragments ? TEXT("Hide") : TEXT("Show"));
	if (bShowAgentFragments)
	{
		CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s Entity Details"), *GetInputHandlerDescription(4), bShowEntityDetails ? TEXT("Hide") : TEXT("Show"));
	}
	else
	{
		CanvasContext.Printf(TEXT("{grey}[%s] Entity Details [enable Agent Fragments]{white}"), *GetInputHandlerDescription(4));
	}
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] Pick Entity"), *GetInputHandlerDescription(3));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] Toggle Picked Actor as Viewer"), *GetInputHandlerDescription(TogglePickedActorAsViewerInputIndex));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s LOD Viewers"), *GetInputHandlerDescription(ToggleDrawViewersInputIndex), bShowViewers ? TEXT("Hide") : TEXT("Show"));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] Reset Actor LOD Viewers"), *GetInputHandlerDescription(ClearViewersInputIndex));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s Entity Overview"), *GetInputHandlerDescription(5), bShowNearEntityOverview ? TEXT("Hide") : TEXT("Show"));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s Entity Avoidance"), *GetInputHandlerDescription(6), bShowNearEntityAvoidance ? TEXT("Hide") : TEXT("Show"));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s Entity Path"), *GetInputHandlerDescription(7), bShowNearEntityPath ? TEXT("Hide") : TEXT("Show"));
	CanvasContext.Printf(TEXT("[{yellow}%s{white}] %s Entity LookAt"), *GetInputHandlerDescription(8), bShowEntityLookAt ? TEXT("Hide") : TEXT("Show"));

	switch (EntityDescriptionVerbosity)
	{
	case EEntityDescriptionVerbosity::Hidden: CanvasContext.Printf(TEXT("[{yellow}%s{white}] Cycle Description Verbosity (none)"), *GetInputHandlerDescription(9)); break;
	case EEntityDescriptionVerbosity::Minimal: CanvasContext.Printf(TEXT("[{yellow}%s{white}] Cycle Description Verbosity (minimal)"), *GetInputHandlerDescription(9));	break;
	case EEntityDescriptionVerbosity::Full: [[fallthrough]];
	case EEntityDescriptionVerbosity::Max: CanvasContext.Printf(TEXT("[{yellow}%s{white}] Cycle Description Verbosity (full)"), *GetInputHandlerDescription(9)); break;
	}
	
	if (IsCategoryLocal() && !IsCategoryAuth())
	{
		// we want to display this line only on clients in client-server environment.
		CanvasContext.Printf(TEXT("[{yellow}%s{white}] Toggle Local/Remote debugging"), *GetInputHandlerDescription(ToggleDebugLocalEntityManagerInputIndex));
	}

	struct FEntityLayoutRect
	{
		FVector2D Min = FVector2D::ZeroVector;
		FVector2D Max = FVector2D::ZeroVector;
		int32 Index = INDEX_NONE;
		float Alpha = 1.0f;
	};

	TArray<FEntityLayoutRect> Layout;

	// The loop below is O(N^2), make sure to keep the N small.
	constexpr int32 MaxDesc = 20;
	const int32 NumDescs = FMath::Min(NearEntityDescriptions.Num(), MaxDesc);
	
	// The labels are assumed to have been ordered in order of importance (i.e. front to back).
	for (int32 Index = 0; Index < NumDescs; Index++)
	{
		const FEntityDescription& Desc = NearEntityDescriptions[Index];
		if (Desc.Description.Len() && CanvasContext.IsLocationVisible(Desc.Location))
		{
			float SizeX = 0, SizeY = 0;
			const FVector2D ScreenLocation = CanvasContext.ProjectLocation(Desc.Location);
			CanvasContext.MeasureString(Desc.Description, SizeX, SizeY);
			
			FEntityLayoutRect Rect;
			Rect.Min = ScreenLocation + FVector2D(0, -SizeY * 0.5f);
			Rect.Max = Rect.Min + FVector2D(SizeX, SizeY);
			Rect.Index = Index;
			Rect.Alpha = 0.0f;

			// Calculate transparency based on how much more important rects are overlapping the new rect.
			const FVector::FReal Area = FMath::Max(0.0, Rect.Max.X - Rect.Min.X) * FMath::Max(0.0, Rect.Max.Y - Rect.Min.Y);
			const FVector::FReal InvArea = Area > KINDA_SMALL_NUMBER ? 1.0 / Area : 0.0;
			FVector::FReal Coverage = 0.0;

			for (const FEntityLayoutRect& Other : Layout)
			{
				// Calculate rect intersection
				const FVector::FReal MinX = FMath::Max(Rect.Min.X, Other.Min.X);
				const FVector::FReal MinY = FMath::Max(Rect.Min.Y, Other.Min.Y);
				const FVector::FReal MaxX = FMath::Min(Rect.Max.X, Other.Max.X);
				const FVector::FReal MaxY = FMath::Min(Rect.Max.Y, Other.Max.Y);

				// return zero area if not overlapping
				const FVector::FReal IntersectingArea = FMath::Max(0.0, MaxX - MinX) * FMath::Max(0.0, MaxY - MinY);
				Coverage += (IntersectingArea * InvArea) * Other.Alpha;
			}

			Rect.Alpha = FloatCastChecked<float>(FMath::Square(1.0 - FMath::Min(Coverage, 1.0)), UE::LWC::DefaultFloatPrecision);
			
			if (Rect.Alpha > KINDA_SMALL_NUMBER)
			{
				Layout.Add(Rect);
			}
		}
	}

	// Render back to front so that the most important item renders at top.
	const FVector2D Padding(5, 5);
	for (int32 Index = Layout.Num() - 1; Index >= 0; Index--)
	{
		const FEntityLayoutRect& Rect = Layout[Index];
		const FEntityDescription& Desc = NearEntityDescriptions[Rect.Index];

		const FVector2D BackgroundPosition(Rect.Min - Padding);
		FCanvasTileItem Background(Rect.Min - Padding, Rect.Max - Rect.Min + Padding * 2.0f, FLinearColor(0.0f, 0.0f, 0.0f, 0.35f * Rect.Alpha));
		Background.BlendMode = SE_BLEND_TranslucentAlphaOnly;
		CanvasContext.DrawItem(Background
			, FloatCastChecked<float>(BackgroundPosition.X, UE::LWC::DefaultFloatPrecision)
			, FloatCastChecked<float>(BackgroundPosition.Y, UE::LWC::DefaultFloatPrecision));

		CanvasContext.PrintAt(FloatCastChecked<float>(Rect.Min.X, UE::LWC::DefaultFloatPrecision)
			, FloatCastChecked<float>(Rect.Min.Y, UE::LWC::DefaultFloatPrecision)
			, FColor::White, Rect.Alpha, Desc.Description);
	}

	FGameplayDebuggerCategory::DrawData(OwnerPC, CanvasContext);
}

void FGameplayDebuggerCategory_Mass::OnToggleDebugLocalEntityManager()
{
	// this code will only execute on locally-controlled categories (as per BindKeyPress's EGameplayDebuggerInputMode::Local
	// parameter). In such a case we don't want to toggle if we're also Auth (there's no client-server relationship here).
	if (IsCategoryAuth())
	{
		return;
	}

	ResetReplicatedData();
	bDebugLocalEntityManager = !bDebugLocalEntityManager;
	bAllowLocalDataCollection = bDebugLocalEntityManager;

	const EGameplayDebuggerInputMode NewInputMode = bDebugLocalEntityManager ? EGameplayDebuggerInputMode::Local : EGameplayDebuggerInputMode::Replicated;
	for (int32 HandlerIndex = 0; HandlerIndex < GetNumInputHandlers(); ++HandlerIndex)
	{
		if (HandlerIndex != ToggleDebugLocalEntityManagerInputIndex)
		{
			GetInputHandler(HandlerIndex).Mode = NewInputMode;
		}
	}

	CachedEntity.Reset();
}

void FGameplayDebuggerCategory_Mass::OnIncreaseSearchRange()
{
	SearchRange = FMath::Clamp(SearchRange * SearchRangeChangeScale, MinSearchRange, MaxSearchRange);
}

void FGameplayDebuggerCategory_Mass::OnDecreaseSearchRange()
{
	SearchRange = FMath::Clamp(SearchRange / SearchRangeChangeScale, MinSearchRange, MaxSearchRange);
}

void FGameplayDebuggerCategory_Mass::OnTogglePickedActorAsViewer()
{
	if (AActor* DebugActor = CachedDebugActor.Get())
	{
		UWorld* World = GetWorldFromReplicator();
		if (UMassLODSubsystem* LODSubsystem = World->GetSubsystem<UMassLODSubsystem>())
		{
			FMassViewerHandle ViewerHandle = LODSubsystem->GetViewerHandleFromActor(*DebugActor);
			if (ViewerHandle.IsValid() == false)
			{
				LODSubsystem->RegisterActorViewer(*DebugActor);
			}
			else
			{
				LODSubsystem->UnregisterActorViewer(*DebugActor);
			}
		}
	}
}

void FGameplayDebuggerCategory_Mass::OnClearActorViewers()
{
	UWorld* World = GetWorldFromReplicator();
	if (UMassLODSubsystem* LODSubsystem = World->GetSubsystem<UMassLODSubsystem>())
	{
		LODSubsystem->DebugUnregisterActorViewer();
	}
}

//-----------------------------------------------------------------------------
// DEPRECATED
//-----------------------------------------------------------------------------
void FGameplayDebuggerCategory_Mass::PickEntity(const APlayerController& OwnerPC, const UWorld& World, FMassEntityManager& EntityManager, const bool bLimitAngle)
{
	FVector ViewLocation = FVector::ZeroVector;
	FVector ViewDirection = FVector::ForwardVector;
	ensureMsgf(GetViewPoint(&OwnerPC, ViewLocation, ViewDirection), TEXT("GetViewPoint is expected to always succeed when passing a valid controller."));

	PickEntity(ViewLocation, ViewDirection, World, EntityManager, bLimitAngle);
}

#endif // WITH_GAMEPLAY_DEBUGGER && WITH_MASSGAMEPLAY_DEBUG

