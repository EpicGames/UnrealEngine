// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IMassAIDebugModule.h"
#if WITH_GAMEPLAY_DEBUGGER
#include "GameplayDebugger.h"
#include "GameplayDebuggerCategory_Mass.h"
#endif // WITH_GAMEPLAY_DEBUGGER


class FMassAIDebug : public IMassAIDebugModule
{
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
};

IMPLEMENT_MODULE(FMassAIDebug, MassAIDebug)

void FMassAIDebug::StartupModule()
{
#if WITH_GAMEPLAY_DEBUGGER && WITH_MASSGAMEPLAY_DEBUG
	IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
	GameplayDebuggerModule.RegisterCategory("Mass", IGameplayDebugger::FOnGetCategory::CreateStatic(&FGameplayDebuggerCategory_Mass::MakeInstance), EGameplayDebuggerCategoryState::EnabledInGameAndSimulate);
	GameplayDebuggerModule.NotifyCategoriesChanged();
#endif
}

void FMassAIDebug::ShutdownModule()
{
#if WITH_GAMEPLAY_DEBUGGER && WITH_MASSGAMEPLAY_DEBUG
	if (IGameplayDebugger::IsAvailable())
	{
		IGameplayDebugger& GameplayDebuggerModule = IGameplayDebugger::Get();
		GameplayDebuggerModule.UnregisterCategory("Mass");
		GameplayDebuggerModule.NotifyCategoriesChanged();
	}
#endif
}



