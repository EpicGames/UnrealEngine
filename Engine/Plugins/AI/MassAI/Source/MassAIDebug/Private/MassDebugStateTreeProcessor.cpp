// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassDebugStateTreeProcessor.h"
#include "MassDebuggerSubsystem.h"
#include "MassStateTreeExecutionContext.h"
#include "MassStateTreeFragments.h"
#include "MassCommonFragments.h"
#include "MassCommonTypes.h"
#include "MassDebugger.h"
#include "MassExecutionContext.h"
#include "VisualLogger/VisualLogger.h"
#include "Engine/World.h"

//----------------------------------------------------------------------//
// UMassDebugStateTreeProcessor
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassDebugStateTreeProcessor)
UMassDebugStateTreeProcessor::UMassDebugStateTreeProcessor()
	: EntityQuery(*this)
{
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Behavior;
	ExecutionOrder.ExecuteAfter.Add(TEXT("MassStateTreeProcessor"));

	// Run on game thread to avoid race condition for AppendSelectedEntityInfo
	bRequiresGameThreadExecution = true;
}

void UMassDebugStateTreeProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassStateTreeInstanceFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddConstSharedRequirement<FMassStateTreeSharedFragment>();
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassDebugStateTreeProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
#if WITH_MASSGAMEPLAY_DEBUG
	UWorld* World = GetWorld();
	if (World == nullptr)
	{
		return;
	}
	
	UMassDebuggerSubsystem* Debugger = World->GetSubsystem<UMassDebuggerSubsystem>();
	if (Debugger == nullptr)
	{
		return;
	}

	UMassStateTreeSubsystem* MassStateTreeSubsystem = World->GetSubsystem<UMassStateTreeSubsystem>();
	if (MassStateTreeSubsystem == nullptr)
	{
		return;
	}

	if (!Debugger->GetSelectedEntity().IsSet() && !UE::Mass::Debug::HasDebugEntities())
	{
		return;
	}
	
	QUICK_SCOPE_CYCLE_COUNTER(UMassDebugStateTreeProcessor_Run);	
	EntityQuery.ForEachEntityChunk(Context, [this, Debugger, MassStateTreeSubsystem](FMassExecutionContext& Context)
	{
		const FMassEntityHandle SelectedEntity = Debugger->GetSelectedEntity();
		const TConstArrayView<FMassStateTreeInstanceFragment> StateTreeInstanceList = Context.GetFragmentView<FMassStateTreeInstanceFragment>();
		const FMassStateTreeSharedFragment& SharedStateTree = Context.GetConstSharedFragment<FMassStateTreeSharedFragment>();
		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();

		const UStateTree* StateTree = SharedStateTree.StateTree;

		// Not reporting error since this processor is a debug tool 
		if (StateTree == nullptr)
		{
			return;
		}
	
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);

			if (Entity != SelectedEntity && !UE::Mass::Debug::IsDebuggingEntity(Entity))
			{
				continue;
			}
			
			const FMassStateTreeInstanceFragment& StateTreeInstance = StateTreeInstanceList[EntityIt];

			FStateTreeInstanceData* InstanceData = MassStateTreeSubsystem->GetInstanceData(StateTreeInstance.InstanceHandle);
			if (InstanceData == nullptr)
			{
				continue;
			}
			
			if (Entity == SelectedEntity)
			{
				FStateTreeReadOnlyExecutionContext StateTreeContext(MassStateTreeSubsystem, StateTree, *InstanceData);
				
#if WITH_GAMEPLAY_DEBUGGER
				Debugger->AppendSelectedEntityInfo(StateTreeContext.GetDebugInfoString());
#endif // WITH_GAMEPLAY_DEBUGGER
			}
				
			FColor EntityColor = FColor::White;
			const bool bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Entity, &EntityColor);
			if (bDisplayDebug)
			{
				const FTransformFragment& Transform = TransformList[EntityIt];
				
				const FVector ZOffset(0,0,50);
				const FVector Position = Transform.GetTransform().GetLocation() + ZOffset;

				FStateTreeReadOnlyExecutionContext StateTreeContext(MassStateTreeSubsystem, StateTree, *InstanceData);

				// State
				UE_VLOG_SEGMENT_THICK(this, LogStateTree, Log, Position, Position + FVector(0,0,50), EntityColor, /*Thickness*/ 2, TEXT("%s %s"),
					*Entity.DebugGetDescription(), *StateTreeContext.GetActiveStateName());
			}
		}
	});
	#endif // WITH_MASSGAMEPLAY_DEBUG
}
