// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Required first for WITH_MASSGAMEPLAY_DEBUG
#include "MassCommonTypes.h"

#if WITH_GAMEPLAY_DEBUGGER && WITH_MASSGAMEPLAY_DEBUG

#include "Mass/EntityHandle.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "HAL/IConsoleManager.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "GameplayDebuggerCategory.h"

struct FMassEntityManager;
class UMassDebuggerSubsystem;
class APlayerController;
class AActor;

class FGameplayDebuggerCategory_Mass : public FGameplayDebuggerCategory
{
	using Super = FGameplayDebuggerCategory;
public:
	FGameplayDebuggerCategory_Mass();
	virtual ~FGameplayDebuggerCategory_Mass();

	static TSharedRef<FGameplayDebuggerCategory> MakeInstance();

protected:
	virtual void CollectData(APlayerController* OwnerPC, AActor* DebugActor) override;
	virtual void DrawData(APlayerController* OwnerPC, FGameplayDebuggerCanvasContext& CanvasContext) override;
	
	void SetCachedEntity(const FMassEntityHandle Entity, const FMassEntityManager& EntityManager);

	void OnToggleArchetypes()
	{
		bShowArchetypes = !bShowArchetypes;
	}

	void OnToggleShapes()
	{
		bShowShapes = !bShowShapes;
	}

	void OnToggleAgentFragments()
	{
		bShowAgentFragments = !bShowAgentFragments;
	}

	void OnPickEntity()
	{
		bPickEntity = true;
	}

	void OnToggleEntityDetails()
	{
		bShowEntityDetails = !bShowEntityDetails;
	}

	void OnToggleNearEntityOverview()
	{
		bShowNearEntityOverview = !bShowNearEntityOverview;
	}

	void OnToggleNearEntityAvoidance()
	{
		bShowNearEntityAvoidance = !bShowNearEntityAvoidance;
	}
	void OnToggleNearEntityPath()
	{
		bShowNearEntityPath = !bShowNearEntityPath;
	}

	void OnToggleEntityLookAt()
	{
		bShowEntityLookAt = !bShowEntityLookAt;
	}

	void OnCycleEntityDescription()
	{
		EntityDescriptionVerbosity = static_cast<EEntityDescriptionVerbosity>(
			(static_cast<uint8>(EntityDescriptionVerbosity) + 1) % static_cast<uint8>(EEntityDescriptionVerbosity::Max));
	}

	void OnToggleDebugLocalEntityManager();
	void OnIncreaseSearchRange();
	void OnDecreaseSearchRange();
	void OnTogglePickedActorAsViewer();
	void OnToggleDrawViewers()
	{
		bShowViewers = !bShowViewers;
	}

	void OnClearActorViewers();
	
	void PickEntity(const FVector& ViewLocation, const FVector& ViewDirection, const UWorld& World, FMassEntityManager& EntityManager, const bool bLimitAngle = true);

	UE_DEPRECATED(5.3, "This flavor of PickEntity has been deprecated. Use the one getting ViewLocation and ViewDirection parameters instead.")
	void PickEntity(const APlayerController& OwnerPC, const UWorld& World, FMassEntityManager& EntityManager, const bool bLimitAngle = true);

	void OnEntitySelected(const FMassEntityManager& EntityManager, const FMassEntityHandle EntityHandle);
	void ClearCachedEntity();

protected:
	enum class EEntityDescriptionVerbosity : uint8
	{
		Hidden,
		Minimal,
		Full,
		Max
	};

	TWeakObjectPtr<AActor> CachedDebugActor;
	FMassEntityHandle CachedEntity;
	EEntityDescriptionVerbosity EntityDescriptionVerbosity = EEntityDescriptionVerbosity::Minimal;

	bool bShowArchetypes;
	bool bShowShapes;
	bool bShowAgentFragments;
	bool bPickEntity;
	bool bShowEntityDetails;
	bool bShowNearEntityOverview;
	bool bShowNearEntityAvoidance;
	bool bShowNearEntityPath;
	bool bShowEntityLookAt;
	bool bMarkEntityBeingDebugged;
	bool bDebugLocalEntityManager;
	bool bShowViewers;
	int32 ToggleDebugLocalEntityManagerInputIndex = INDEX_NONE;
	int32 TogglePickedActorAsViewerInputIndex = INDEX_NONE;
	int32 ToggleDrawViewersInputIndex = INDEX_NONE;
	int32 ClearViewersInputIndex = INDEX_NONE;
	float SearchRange = 25000.f;

	struct FEntityDescription
	{
		FEntityDescription() = default;
		FEntityDescription(const float InScore, const FVector& InLocation, const FString& InDescription) : Score(InScore), Location(InLocation), Description(InDescription) {}

		float Score = 0.0f;
		FVector Location = FVector::ZeroVector;
		FString Description;
	};
	TArray<FEntityDescription> NearEntityDescriptions;

	static TArray<FAutoConsoleCommandWithWorld> ConsoleCommands;
	DECLARE_MULTICAST_DELEGATE_OneParam(FOnConsoleCommandBroadcastDelegate, UWorld*);
	static FOnConsoleCommandBroadcastDelegate OnToggleArchetypesBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleShapesBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleAgentFragmentsBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnPickEntityBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleEntityDetailsBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleNearEntityOverviewBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleNearEntityAvoidanceBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleNearEntityPathBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleEntityLookAtBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnCycleEntityDescriptionBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleDebugLocalEntityManagerBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnTogglePickedActorAsViewerBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnToggleDrawViewersBroadcast;
	static FOnConsoleCommandBroadcastDelegate OnClearActorViewersBroadcast;

	using FDelegateHandlePair = TPair<FOnConsoleCommandBroadcastDelegate*, FDelegateHandle>;
	TArray<FDelegateHandlePair> ConsoleCommandHandles;

	FDelegateHandle OnEntitySelectedHandle;

	static constexpr float MaxSearchRange = 1000000.f;
	static constexpr float MinSearchRange = 1.f;
	static constexpr float SearchRangeChangeScale = 2.f;
};

#endif // WITH_GAMEPLAY_DEBUGGER && WITH_MASSGAMEPLAY_DEBUG
