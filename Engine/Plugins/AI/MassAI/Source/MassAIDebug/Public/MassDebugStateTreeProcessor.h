// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "MassDebugStateTreeProcessor.generated.h"

#define UE_API MASSAIDEBUG_API

struct FMassEntityManager;
struct FMassEntityQuery;
struct FMassExecutionContext;

UCLASS(MinimalAPI)
class UMassDebugStateTreeProcessor : public UMassProcessor
{
	GENERATED_BODY()

protected:
	UE_API UMassDebugStateTreeProcessor();

	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
