// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassReplicationPathHandlers.h"
#include "MassEntityQuery.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassSimulationSubsystem.h"
#include "MassZoneGraphNavigationUtils.h"
#include "ZoneGraphSubsystem.h"
#include "GameFramework/GameStateBase.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassReplicationPathHandlers)

void FMassReplicationProcessorPathHandler::AddRequirements(FMassEntityQuery& InQuery)
{
	InQuery.AddRequirement<FMassZoneGraphPathRequestFragment>(EMassFragmentAccess::ReadWrite);
	InQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadWrite);
	InQuery.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadWrite);
}

void FMassReplicationProcessorPathHandler::CacheFragmentViews(FMassExecutionContext& ExecContext)
{
	PathRequestList = ExecContext.GetMutableFragmentView<FMassZoneGraphPathRequestFragment>();
	MoveTargetList = ExecContext.GetMutableFragmentView<FMassMoveTargetFragment>();
	LaneLocationList = ExecContext.GetMutableFragmentView<FMassZoneGraphLaneLocationFragment>();
}

void FMassReplicationProcessorPathHandler::AddEntity(const int32 EntityIdx, FReplicatedAgentPathData& InOutReplicatedPathData) const
{
	const FMassZoneGraphPathRequestFragment& RequestFragment = PathRequestList[EntityIdx];
	const FMassMoveTargetFragment& MoveTargetFragment = MoveTargetList[EntityIdx];
	const FMassZoneGraphLaneLocationFragment& LaneLocationFragment = LaneLocationList[EntityIdx];

	InOutReplicatedPathData = FReplicatedAgentPathData(RequestFragment, MoveTargetFragment, LaneLocationFragment);
}

FReplicatedAgentPathData::FReplicatedAgentPathData(const FMassZoneGraphPathRequestFragment& RequestFragment,
	const FMassMoveTargetFragment& MoveTargetFragment,
	const FMassZoneGraphLaneLocationFragment& LaneLocationFragment)
{
	// Move target
	ActionID = MoveTargetFragment.GetCurrentActionID();
	Action = MoveTargetFragment.GetCurrentAction();
	ActionServerStartTime = MoveTargetFragment.GetCurrentActionStartTime();
	DesiredSpeed = MoveTargetFragment.DesiredSpeed;

	// Lane Location
	LaneHandle = LaneLocationFragment.LaneHandle;
	DistanceAlongLane = LaneLocationFragment.DistanceAlongLane;
	LaneLength = LaneLocationFragment.LaneLength;

	// Path request
	PathRequest = RequestFragment.PathRequest;
}

void FReplicatedAgentPathData::InitEntity(const UWorld& InWorld,
										  const FMassEntityView& InEntityView,
										  FMassZoneGraphLaneLocationFragment& OutLaneLocation,
										  FMassMoveTargetFragment& OutMoveTarget,
										  FMassZoneGraphPathRequestFragment& OutActionRequest) const
{
	const FMassEntityHandle Entity = InEntityView.GetEntity();

	const UZoneGraphSubsystem* ZoneGraphSubsystem = InWorld.GetSubsystem<UZoneGraphSubsystem>();
	const UMassSimulationSubsystem* SimulationSubsystem = InWorld.GetSubsystem<UMassSimulationSubsystem>();
	if (ZoneGraphSubsystem == nullptr || SimulationSubsystem == nullptr)
	{
		UE_CVLOG(ZoneGraphSubsystem == nullptr, &InWorld, LogMassNavigation, Error, TEXT("Entity [%s] no ZoneGraphSubsystem to process request %s"),
			*Entity.DebugGetDescription(), *PathRequest.ToString());
		UE_CVLOG(SimulationSubsystem == nullptr, &InWorld, LogMassNavigation, Error, TEXT("Entity [%s] no MassSimulationSubsystem to process request %s"),
			*Entity.DebugGetDescription(), *PathRequest.ToString());
		return;
	}

	UE_VLOG(SimulationSubsystem, LogMassNavigation, Log, TEXT("%s InitEntity"), *Entity.DebugGetDescription());

	// Setup initial lane location
	OutLaneLocation.LaneHandle = LaneHandle;
	OutLaneLocation.DistanceAlongLane = DistanceAlongLane;
	OutLaneLocation.LaneLength = LaneLength;

	// Setup initial move target
	FZoneGraphLaneLocation LaneLocation;
	ZoneGraphSubsystem->CalculateLocationAlongLane(LaneHandle, DistanceAlongLane, LaneLocation);
	OutMoveTarget.DesiredSpeed = DesiredSpeed;
	OutMoveTarget.Center = LaneLocation.Position;
	OutMoveTarget.Forward = LaneLocation.Tangent;
	OutMoveTarget.DistanceToGoal = 0.0f;
	OutMoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
	OutMoveTarget.SlackRadius = 0.0f;

	// Setup initial action request
	OutActionRequest.PathRequest = PathRequest;

	ApplyToEntity(InWorld, InEntityView);
}

void FReplicatedAgentPathData::ApplyToEntity(const UWorld& InWorld, const FMassEntityView& InEntityView) const
{
	const FMassEntityHandle Entity = InEntityView.GetEntity();
	FMassMoveTargetFragment& MoveTarget = InEntityView.GetFragmentData<FMassMoveTargetFragment>();
	if (MoveTarget.GetCurrentActionID() == ActionID)
	{
		return;
	}

	const UZoneGraphSubsystem* ZoneGraphSubsystem = InWorld.GetSubsystem<UZoneGraphSubsystem>();
	const UMassSimulationSubsystem* SimulationSubsystem = InWorld.GetSubsystem<UMassSimulationSubsystem>();
	if (ZoneGraphSubsystem == nullptr || SimulationSubsystem == nullptr)
	{
		UE_CVLOG(ZoneGraphSubsystem == nullptr, &InWorld, LogMassNavigation, Error, TEXT("Entity [%s] no ZoneGraphSubsystem to process request %s"),
			*Entity.DebugGetDescription(), *PathRequest.ToString());
		UE_CVLOG(SimulationSubsystem == nullptr, &InWorld, LogMassNavigation, Error, TEXT("Entity [%s] no MassSimulationSubsystem to process request %s"),
			*Entity.DebugGetDescription(), *PathRequest.ToString());
		return;
	}

	UE_VLOG(SimulationSubsystem, LogMassNavigation, Log, TEXT("Entity [%s] apply replicated data to entity"), *Entity.DebugGetDescription());

	FMassZoneGraphShortPathFragment& ShortPath = InEntityView.GetFragmentData<FMassZoneGraphShortPathFragment>();
	FMassZoneGraphCachedLaneFragment& CachedLane = InEntityView.GetFragmentData<FMassZoneGraphCachedLaneFragment>();
	FMassZoneGraphLaneLocationFragment& LaneLocation = InEntityView.GetFragmentData<FMassZoneGraphLaneLocationFragment>();
	const FAgentRadiusFragment& AgentRadius = InEntityView.GetFragmentData<FAgentRadiusFragment>();

	MoveTarget.CreateReplicatedAction(Action, ActionID, InWorld.GetTimeSeconds(), ActionServerStartTime);
	MoveTarget.DesiredSpeed = DesiredSpeed;

	// Force current lane to build same path as server
	if (LaneLocation.LaneHandle != LaneHandle)
	{
		UE_VLOG(SimulationSubsystem, LogMassNavigation, Verbose, TEXT("Entity [%s] Force lane location from %s - %.1f to %s - %.1f to build similar path."),
			*Entity.DebugGetDescription(),
			*LaneLocation.LaneHandle.ToString(), LaneLocation.DistanceAlongLane,
			*LaneHandle.ToString(),	DistanceAlongLane);

		LaneLocation.LaneHandle = LaneHandle;
		LaneLocation.DistanceAlongLane = DistanceAlongLane;
		LaneLocation.LaneLength = LaneLength;
	}

	switch (Action)
	{
	case EMassMovementAction::Stand:
		UE::MassNavigation::ActivateActionStand(InWorld, SimulationSubsystem, Entity, *ZoneGraphSubsystem, LaneLocation, DesiredSpeed.Get(), MoveTarget, ShortPath, CachedLane);
		break;
	case EMassMovementAction::Move:
		UE::MassNavigation::ActivateActionMove(InWorld, SimulationSubsystem, Entity, *ZoneGraphSubsystem, LaneLocation, PathRequest, AgentRadius.Radius, DesiredSpeed.Get(), MoveTarget, ShortPath, CachedLane);
		break;
	case EMassMovementAction::Animate:
		UE::MassNavigation::ActivateActionAnimate(InWorld, SimulationSubsystem, Entity, MoveTarget);
		break;
	default:
		ensureMsgf(false, TEXT("Unhandled action type: %s"), *UEnum::GetValueAsString(Action));
	}
}
