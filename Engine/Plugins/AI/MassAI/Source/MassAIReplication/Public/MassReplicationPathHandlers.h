// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassReplicationTypes.h"
#include "MassEntityView.h"
#include "MassClientBubbleHandler.h"
#include "MassNavigationFragments.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassReplicationPathHandlers.generated.h"

#define UE_API MASSAIREPLICATION_API

struct FMassEntityQuery;

//////////////////////////////////////////////////////////////////////////// FReplicatedAgentPathData ////////////////////////////////////////////////////////////////////////////
/**
 * To replicate path following make a member variable of this class in your FReplicatedAgentBase derived class. In the FReplicatedAgentBase derived class you must also provide an accessor function
 * FReplicatedAgentPathData& GetReplicatedPathDataMutable().
 */
USTRUCT()
struct FReplicatedAgentPathData
{
	GENERATED_BODY()

	friend class FMassClientBubblePathHandlerBase;

	FReplicatedAgentPathData() = default;
	UE_API explicit FReplicatedAgentPathData(const FMassZoneGraphPathRequestFragment& RequestFragment,
		const FMassMoveTargetFragment& MoveTargetFragment,
		const FMassZoneGraphLaneLocationFragment& LaneLocationFragment);

	UE_API void InitEntity(const UWorld& InWorld,
					const FMassEntityView& InEntityView,
					FMassZoneGraphLaneLocationFragment& OutLaneLocation,
					FMassMoveTargetFragment& OutMoveTarget,
					FMassZoneGraphPathRequestFragment& OutActionRequest) const;

	UE_API void ApplyToEntity(const UWorld& InWorld, const FMassEntityView& InEntityView) const;

	UPROPERTY(Transient)
	mutable FZoneGraphShortPathRequest PathRequest;

	/** Handle to current lane. */
	UPROPERTY(Transient)
	FZoneGraphLaneHandle LaneHandle;

	/** Server time in seconds when the action started. */
	UPROPERTY(Transient)
	double ActionServerStartTime = 0.;

	/** Distance along current lane. */
	UPROPERTY(Transient)
	float DistanceAlongLane = 0.0f;

	/** Cached lane length, used for clamping and testing if at end of lane. */
	UPROPERTY(Transient)
	float LaneLength = 0.0f;

	/** Requested movement speed. */
	UPROPERTY(Transient)
	FMassInt16Real DesiredSpeed = FMassInt16Real(0.0f);

	UPROPERTY(Transient)
	uint16 ActionID = 0;

	/** Current movement action. */
	UPROPERTY(Transient)
	EMassMovementAction Action = EMassMovementAction::Move;
};

//////////////////////////////////////////////////////////////////////////// TMassClientBubblePathHandler ////////////////////////////////////////////////////////////////////////////
/**
 * To replicate path following make a member variable of this class in your TClientBubbleHandlerBase derived class. This class is a friend of TMassClientBubblePathHandler.
 * It is meant to have access to the protected data declared there.
 */
template<typename AgentArrayItem>
class TMassClientBubblePathHandler
{
public:
	TMassClientBubblePathHandler(TClientBubbleHandlerBase<AgentArrayItem>& InOwnerHandler)
		: OwnerHandler(InOwnerHandler)
	{}

#if UE_REPLICATION_COMPILE_SERVER_CODE
	/** Sets the path data in the client bubble on the server */
	void SetBubblePathData(const FMassReplicatedAgentHandle Handle,
		const FMassZoneGraphPathRequestFragment& PathRequest,
		const FMassMoveTargetFragment& MoveTargetFragment,
		const FMassZoneGraphLaneLocationFragment& LaneLocationFragment);
#endif // UE_REPLICATION_COMPILE_SERVER_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
	/**
	 * When entities are spawned in Mass by the replication system on the client, a spawn query is used to set the data on the spawned entities.
	 * The following functions are used to configure the query and then set that data for path following.
	 */
	static void AddRequirementsForSpawnQuery(FMassEntityQuery& InQuery);
	void CacheFragmentViewsForSpawnQuery(FMassExecutionContext& InExecContext);
	void ClearFragmentViewsForSpawnQuery();

	void SetSpawnedEntityData(const FMassEntityView& EntityView, const FReplicatedAgentPathData& ReplicatedPathData, const int32 EntityIdx) const;

	/** Call this when an Entity that has already been spawned is modified on the client */
	void SetModifiedEntityData(const FMassEntityView& EntityView, const FReplicatedAgentPathData& ReplicatedPathData) const;
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

protected:
	TArrayView<FMassZoneGraphPathRequestFragment> PathRequestList;
	TArrayView<FMassMoveTargetFragment> MoveTargetList;
	TArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList;

	TClientBubbleHandlerBase<AgentArrayItem>& OwnerHandler;
};

#if UE_REPLICATION_COMPILE_SERVER_CODE
template<typename AgentArrayItem>
void TMassClientBubblePathHandler<AgentArrayItem>::SetBubblePathData(const FMassReplicatedAgentHandle Handle,
	const FMassZoneGraphPathRequestFragment& PathRequestFragment,
	const FMassMoveTargetFragment& MoveTargetFragment,
	const FMassZoneGraphLaneLocationFragment& LaneLocationFragment)
{
	AgentArrayItem& Item = OwnerHandler.GetAgentItem(Handle);

	checkf(Item.Agent.GetNetID().IsValid(), TEXT("Pos should not be updated on FCrowdFastArrayItem's that have an Invalid ID! First Add the Agent!"));

	// GetReplicatedPathDataMutable() must be defined in your FReplicatedAgentBase derived class
	FReplicatedAgentPathData& ReplicatedPath = Item.Agent.GetReplicatedPathDataMutable();

	if (ReplicatedPath.ActionID != MoveTargetFragment.GetCurrentActionID())
	{
		ReplicatedPath = FReplicatedAgentPathData(PathRequestFragment, MoveTargetFragment, LaneLocationFragment);

		OwnerHandler.MarkItemDirty(Item);
	}
}
#endif //UE_REPLICATION_COMPILE_SERVER_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubblePathHandler<AgentArrayItem>::AddRequirementsForSpawnQuery(FMassEntityQuery& InQuery)
{
	InQuery.AddRequirement<FMassZoneGraphPathRequestFragment>(EMassFragmentAccess::ReadWrite);
	InQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadWrite);
	InQuery.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadWrite);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubblePathHandler<AgentArrayItem>::CacheFragmentViewsForSpawnQuery(FMassExecutionContext& InExecContext)
{
	PathRequestList = InExecContext.GetMutableFragmentView<FMassZoneGraphPathRequestFragment>();
	MoveTargetList = InExecContext.GetMutableFragmentView<FMassMoveTargetFragment>();
	LaneLocationList = InExecContext.GetMutableFragmentView<FMassZoneGraphLaneLocationFragment>();
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubblePathHandler<AgentArrayItem>::ClearFragmentViewsForSpawnQuery()
{
	LaneLocationList = TArrayView<FMassZoneGraphLaneLocationFragment>();
	MoveTargetList = TArrayView<FMassMoveTargetFragment>();
	PathRequestList = TArrayView<FMassZoneGraphPathRequestFragment>();
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubblePathHandler<AgentArrayItem>::SetSpawnedEntityData(const FMassEntityView& EntityView, const FReplicatedAgentPathData& ReplicatedPathData, const int32 EntityIdx) const
{
	UWorld* World = OwnerHandler.GetWorld();
	check(World);
	ReplicatedPathData.InitEntity(*World, EntityView, LaneLocationList[EntityIdx], MoveTargetList[EntityIdx], PathRequestList[EntityIdx]);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

#if UE_REPLICATION_COMPILE_CLIENT_CODE
template<typename AgentArrayItem>
void TMassClientBubblePathHandler<AgentArrayItem>::SetModifiedEntityData(const FMassEntityView& EntityView, const FReplicatedAgentPathData& ReplicatedPathData) const
{
	UWorld* World = OwnerHandler.GetWorld();
	check(World);
	ReplicatedPathData.ApplyToEntity(*World, EntityView);
}
#endif // UE_REPLICATION_COMPILE_CLIENT_CODE

//////////////////////////////////////////////////////////////////////////// FMassReplicationProcessorPathHandler ////////////////////////////////////////////////////////////////////////////
/**
 * Used to replicate path following by your UMassReplicationProcessorBase derived class. This class should only get used on the server.
 * @todo add #if UE_REPLICATION_COMPILE_SERVER_CODE
 */
class FMassReplicationProcessorPathHandler
{
public:
	/** Adds the requirements for the path following to the query. */
	static UE_API void AddRequirements(FMassEntityQuery& InQuery);

	/** Cache any component views you want to, this will get called before we iterate through entities. */
	UE_API void CacheFragmentViews(FMassExecutionContext& ExecContext);

	/**
	 * Set the replicated path data when we are adding an entity to the client bubble.
	 * @param EntityIdx the index of the entity in fragment views that have been cached.
	 * @param InOUtReplicatedPathData the data to set.
	 */
	UE_API void AddEntity(const int32 EntityIdx, FReplicatedAgentPathData& InOUtReplicatedPathData) const;

	/**
	 * Set the replicated path data when we are modifying an entity that already exists in the client bubble.
	 * @param Handle to the agent in the TMassClientBubbleHandler (that TMassClientBubblePathHandler is a member variable of).
	 * @param EntityIdx the index of the entity in fragment views that have been cached.
	 * @param BubblePathHandler handler to actually set the data in the client bubble
	 * @param bLastClient means it safe to reset any dirtiness
	 */
	template<typename AgentArrayItem>
	void ModifyEntity(const FMassReplicatedAgentHandle Handle, const int32 EntityIdx, TMassClientBubblePathHandler<AgentArrayItem>& BubblePathHandler, bool bLastClient);

	TArrayView<FMassZoneGraphPathRequestFragment> PathRequestList;
	TArrayView<FMassMoveTargetFragment> MoveTargetList;
	TArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList;
};

template<typename AgentArrayItem>
void FMassReplicationProcessorPathHandler::ModifyEntity(const FMassReplicatedAgentHandle Handle, const int32 EntityIdx, TMassClientBubblePathHandler<AgentArrayItem>& BubblePathHandler, bool bLastClient)
{
	const FMassZoneGraphPathRequestFragment& PathRequest = PathRequestList[EntityIdx];
	FMassMoveTargetFragment& MoveTargetFragment = MoveTargetList[EntityIdx];
	const FMassZoneGraphLaneLocationFragment& LaneLocationFragment = LaneLocationList[EntityIdx];

	if (MoveTargetFragment.GetNetDirty())
	{
		BubblePathHandler.SetBubblePathData(Handle, PathRequest, MoveTargetFragment, LaneLocationFragment);
		if (bLastClient)
		{
			MoveTargetFragment.ResetNetDirty();
		}
	}
}

#undef UE_API
