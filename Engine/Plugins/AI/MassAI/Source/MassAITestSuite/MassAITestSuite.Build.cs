// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

namespace UnrealBuildTool.Rules
{
	public class MassAITestSuite : ModuleRules
	{
		public MassAITestSuite(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

			CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Warning;

			PublicIncludePaths.AddRange(
				new string[] {
				}
			);

			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Engine",
					"AIModule",
					"AITestSuite",
					"MassActors",
					"MassAIBehavior",
					"MassCommon",
					"MassNavigation",
					"MassAIReplication",
					"MassSmartObjects",
					"MassSpawner",
					"MassRepresentation"
				}
			);

			DynamicallyLoadedModuleNames.AddRange(
				new string[] {
                    // ... add any modules that your module loads dynamically here ...
                }
			);

			if (Target.bBuildEditor == true)
			{
				PrivateDependencyModuleNames.Add("UnrealEd");
			}
		}
	}
}