// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassAITestSuiteModule.h"

#define LOCTEXT_NAMESPACE "MassAITestSuite"

class FMassAITestSuiteModule : public IMassAITestSuiteModule
{
};

IMPLEMENT_MODULE(FMassAITestSuiteModule, MassAITestSuite)

#undef LOCTEXT_NAMESPACE
