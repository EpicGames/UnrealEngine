// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MassNavMeshNavigation : ModuleRules
	{
		public MassNavMeshNavigation(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;
			
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"AIModule",
					"MassCore",
					"MassEntity",
					"Core",
					"CoreUObject",
					"Engine",
					"MassCommon",
					"MassLOD",
					"MassSignals",
					"MassSpawner",
					"MassMovement",
					"MassNavigation",
					"NavCorridor"
				}
			);
		}
	}
}