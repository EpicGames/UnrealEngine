// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavMeshNavigationBoundaryProcessor.h"

#include "MassNavMeshNavigationTypes.h"
#include "MassCommonTypes.h"
#include "MassLODFragments.h"
#include "MassNavigationFragments.h"
#include "MassNavMeshNavigationFragments.h"
#include "Avoidance/MassAvoidanceFragments.h"
#include "MassDebugger.h"
#include "VisualLogger/VisualLogger.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavMeshNavigationBoundaryProcessor)

UMassNavMeshNavigationBoundaryProcessor::UMassNavMeshNavigationBoundaryProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;

	bAutoRegisterWithProcessingPhases = true;
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Avoidance);
}

void UMassNavMeshNavigationBoundaryProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassNavMeshShortPathFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	
	EntityQuery.AddRequirement<FMassNavMeshBoundaryFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassNavigationEdgesFragment>(EMassFragmentAccess::ReadWrite);	// to output edges for avoidance
}

void UMassNavMeshNavigationBoundaryProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UMassNavMeshNavigationBoundaryProcessor);

	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		TConstArrayView<FMassNavMeshShortPathFragment> ShortPathList = Context.GetFragmentView<FMassNavMeshShortPathFragment>();
		TConstArrayView<FMassMoveTargetFragment> MovementTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();

		TArrayView<FMassNavMeshBoundaryFragment> NavmeshBoundaryList = Context.GetMutableFragmentView<FMassNavMeshBoundaryFragment>();
		TArrayView<FMassNavigationEdgesFragment> EdgesList = Context.GetMutableFragmentView<FMassNavigationEdgesFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassNavMeshShortPathFragment& ShortPath = ShortPathList[EntityIt];
			const FMassMoveTargetFragment& MovementTarget = MovementTargetList[EntityIt];
			FMassNavMeshBoundaryFragment& NavmeshBoundary = NavmeshBoundaryList[EntityIt];
			FMassNavigationEdgesFragment& Edges = EdgesList[EntityIt];

			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);

			// First check if we moved enough for an update.
			const FVector::FReal DeltaDistSquared = FVector::DistSquared(MovementTarget.Center, NavmeshBoundary.LastUpdatePosition);
			constexpr FVector::FReal UpdateDistanceThresholdSquared = FMath::Square(50.);

			bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG
			FColor EntityColor = FColor::Black;
			bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Entity, &EntityColor);

			if (bDisplayDebug)
			{
				// Draw LastUpdatePosition
				const FVector ZOffset(0,0,10);
				constexpr float Radius = 5.f;
				UE_VLOG_WIRECIRCLE(this, LogMassNavMeshNavigation, Verbose, NavmeshBoundary.LastUpdatePosition + ZOffset,
					FVector(0,0,1), Radius, FColor::Blue, TEXT("Boundary update"));
			}
#endif // WITH_MASSGAMEPLAY_DEBUG

			if (DeltaDistSquared < UpdateDistanceThresholdSquared)
			{
				// Not moved enough
				continue;
			}

			NavmeshBoundary.LastUpdatePosition = MovementTarget.Center;
			Edges.AvoidanceEdges.Reset();

			if (ShortPath.NumPoints < 2)
			{
				// Nothing to do
				continue;
			}

			// Make environment edges from the short path.
			Edges.bExtrudedEdges = true;
			for (int32 Index = 0; Index+1 < ShortPath.NumPoints && 2*Index < Edges.MaxEdgesCount; Index++)
			{
				const FMassNavMeshPathPoint& Portal = ShortPath.Points[Index];
				const FMassNavMeshPathPoint& NextPortal = ShortPath.Points[Index+1];

				// If the points are too close, just ignore the edges.

				if (!(NextPortal.Left - Portal.Left).IsNearlyZero())
				{
					// Left side: reverse start and end to keep the normal inside.
					Edges.AvoidanceEdges.Add(FNavigationAvoidanceEdge(NextPortal.Left, Portal.Left));
				}

				if (!(Portal.Right - NextPortal.Right).IsNearlyZero())
				{
					Edges.AvoidanceEdges.Add(FNavigationAvoidanceEdge(Portal.Right, NextPortal.Right));
				}
			}
		}
	});
}
