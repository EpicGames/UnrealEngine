// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavMeshNavigationFragments.h"

#include "AI/Navigation/NavigationTypes.h"
#include "NavCorridor.h"

// Make a new short corridor

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavMeshNavigationFragments)
bool FMassNavMeshShortPathFragment::RequestShortPath(const TSharedPtr<FNavCorridor>& NavCorridor, const int32 InNavCorridorStartIndex, const uint8 InNumLeadingPoints,
	const float InEndReachedDistance)
{
	Reset();
	EndReachedDistance = InEndReachedDistance;

	TArray<FNavCorridorPortal>& NavCorridorPortals = NavCorridor->Portals;
	int32 PortalNum = NavCorridorPortals.Num();

	// Copy Portals from the StartIndex
	for(; NumPoints < MaxPoints && (InNavCorridorStartIndex + NumPoints) < PortalNum; NumPoints++)
	{
		const FNavCorridorPortal& SourcePortal = NavCorridorPortals[InNavCorridorStartIndex + NumPoints];
		
		FMassNavMeshPathPoint& Point = Points[NumPoints];
		Point.Left = SourcePortal.Left;
		Point.Right = SourcePortal.Right;
		Point.Position = SourcePortal.Location;
	}

	checkf(NumPoints >= 2, TEXT("Path should have at least 2 points at this stage, has %d."), NumPoints);

	// If we are not done with the main path, mark as partial.
	const int32 LastNavPathIndex = PortalNum-1;
	if (InNavCorridorStartIndex + (NumPoints-1) < LastNavPathIndex)
	{
		bPartialResult = true;
		ensure(NumPoints == MaxPoints);
	}

	// Calculate movement distance and tangent at each point.
	float PathDistance = 0.f;
	Points[0].Distance.Set(PathDistance);
	Points[0].Tangent = FMassSnorm8Vector2D((Points[1].Position - Points[0].Position).GetSafeNormal());
	
	for (uint8 PointIndex = 1; PointIndex < NumPoints; PointIndex++)
	{
		FMassNavMeshPathPoint& PrevPoint = Points[PointIndex - 1];
		FMassNavMeshPathPoint& Point = Points[PointIndex];
		const FVector& PrevPosition = PrevPoint.Position;
		const FVector& Position = Point.Position;
		const float DeltaDistance = FloatCastChecked<float>(FVector::Dist(PrevPosition, Position), UE::LWC::DefaultFloatPrecision);
		PathDistance += DeltaDistance;
		Point.Distance.Set(PathDistance);

		// Set tangent of Point
		if (PointIndex + 1 < NumPoints)
		{
			FMassNavMeshPathPoint& NextPoint = Points[PointIndex + 1];
			Point.Tangent = FMassSnorm8Vector2D(0.5 * ((Point.Position - PrevPoint.Position).GetSafeNormal() + (NextPoint.Position - Point.Position).GetSafeNormal()));
		}
		else
		{
			// Last point
			Point.Tangent = FMassSnorm8Vector2D((Point.Position - PrevPoint.Position).GetSafeNormal());
		}
	}

	// Update ProgressDistance to account for leading points. 
	MoveTargetProgressDistance = Points[InNumLeadingPoints].Distance.Get();
	
	bInitialized = true;
	
	return true;
}
