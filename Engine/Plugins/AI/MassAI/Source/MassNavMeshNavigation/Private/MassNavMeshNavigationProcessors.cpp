// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavMeshNavigationProcessors.h"
#include "MassCommonFragments.h"
#include "Avoidance/MassAvoidanceFragments.h"
#include "MassCommonTypes.h"
#include "MassDebugger.h"
#include "MassExecutionContext.h"
#include "MassNavMeshNavigationFragments.h"
#include "MassNavMeshNavigationTypes.h"
#include "MassNavigationFragments.h"
#include "MassSignalSubsystem.h"
#include "MassSimulationLOD.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavMeshNavigationProcessors)


UMassNavMeshPathFollowProcessor::UMassNavMeshPathFollowProcessor()
	: EntityQuery_Conditional(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Tasks;
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Avoidance);
}

void UMassNavMeshPathFollowProcessor::InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& EntityManager)
{
	Super::InitializeInternal(Owner, EntityManager);
	SignalSubsystem = UWorld::GetSubsystem<UMassSignalSubsystem>(Owner.GetWorld());
}

void UMassNavMeshPathFollowProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery_Conditional.AddRequirement<FMassNavMeshShortPathFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);

	// @todo: validate LOD and variable ticking
	EntityQuery_Conditional.AddRequirement<FMassSimulationLODFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddRequirement<FMassSimulationVariableTickFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
}

void UMassNavMeshPathFollowProcessor::CheckEndOfPathReached(FMassNavMeshShortPathFragment& ShortPath, 
	const FVector& EntityLocation, const uint8 EndPointIndex, const FMassEntityHandle& Entity, bool bDisplayDebug)
{
	ensure(!ShortPath.bPartialResult);

	// EndReachedDistance is expected to be a small value.
	// The goal is to trigger the path as done once we cross the finish line even if the entity is far from the path segment.
	// For big values this would mean backtracking on multiple segments. We don't want to waste time doing that, and we would be
	// limited by the size short path anyway, so we just do a rough check for big EndReachedDistance values.
	const float DistanceToEndSquared = (EntityLocation - ShortPath.Points[EndPointIndex].Position).SquaredLength();
	if (DistanceToEndSquared < FMath::Square(ShortPath.EndReachedDistance))
	{
		UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Log,
			TEXT("Entity [%s]   Finished path follow on short path, end of path."), *Entity.DebugGetDescription());
		ShortPath.bDone = true;
		return;
	}

	// Normal case: compute distance along final segment to detect if entity reached the end.
	if (ShortPath.MoveTargetProgressDistance > ShortPath.Points[EndPointIndex-1].Distance.Get())
	{
		const float SegmentSize = ShortPath.Points[EndPointIndex].Distance.Get() - ShortPath.Points[EndPointIndex-1].Distance.Get();
		const FVector PreviousPointLocation = ShortPath.Points[EndPointIndex-1].Position;
		const FVector PathDir = (ShortPath.Points[EndPointIndex].Position - PreviousPointLocation).GetSafeNormal();

		const FVector RelativeLocation = EntityLocation - PreviousPointLocation;
		const float ProjectionOnSegment = RelativeLocation.Dot(PathDir);
		if (ProjectionOnSegment > (SegmentSize - ShortPath.EndReachedDistance))
		{
			UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Log,
				TEXT("Entity [%s]   Finished path follow on short path, end of path. Segment %i-%i"), *Entity.DebugGetDescription(), EndPointIndex-1, EndPointIndex);
			ShortPath.bDone = true;
		}
	}
}

void UMassNavMeshPathFollowProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	if (!SignalSubsystem)
	{
		return;
	}
	
	TArray<FMassEntityHandle> EntitiesToSignalPathDone;

	EntityQuery_Conditional.ForEachEntityChunk(Context, [this, &EntitiesToSignalPathDone](FMassExecutionContext& Context)
	{
		const TArrayView<FMassNavMeshShortPathFragment> ShortPathList = Context.GetMutableFragmentView<FMassNavMeshShortPathFragment>();
		const TArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetMutableFragmentView<FMassMoveTargetFragment>();
		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();

		// @todo: validate LOD and variable ticking
		const TConstArrayView<FMassSimulationLODFragment> SimLODList = Context.GetFragmentView<FMassSimulationLODFragment>();
		const bool bHasLOD = (SimLODList.Num() > 0);

		const TConstArrayView<FMassSimulationVariableTickFragment> SimVariableTickList = Context.GetFragmentView<FMassSimulationVariableTickFragment>();
		const float WorldDeltaTime = Context.GetDeltaTimeSeconds();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassNavMeshShortPathFragment& ShortPath = ShortPathList[EntityIt];
			FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);

			bool bDisplayDebug = false;
#if WITH_MASSGAMEPLAY_DEBUG // this will result in bDisplayDebug == false and disabling of all the vlogs below
			bDisplayDebug = UE::Mass::Debug::IsDebuggingEntity(Entity);
#endif // WITH_MASSGAMEPLAY_DEBUG

			// Must have at least two points to interpolate.
			if (MoveTarget.GetCurrentAction() == EMassMovementAction::Move && ShortPath.NumPoints >= 2)
			{
				UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Verbose,
					TEXT("Entity [%s] Updating navmesh path following"), *Entity.DebugGetDescription());
				
				const bool bWasDone = ShortPath.IsDone();

				// Note: this should be in sync with the logic in apply velocity.
				const bool bHasSteering = (bHasLOD == false) || (SimLODList[EntityIt].LOD != EMassLOD::Off);

				if (!bHasSteering || !MoveTarget.bSteeringFallingBehind)
				{
					// Update progress
					ShortPath.MoveTargetProgressDistance += MoveTarget.DesiredSpeed.Get() * WorldDeltaTime;
				}
				
				if (!bWasDone)
				{
					const FTransformFragment& TransformFragment = TransformList[EntityIt];
					const FVector EntityLocation = TransformFragment.GetTransform().GetLocation();
					
					const uint8 LastIndex = ShortPath.NumPoints - 1;

					// Point index on the short path where we should update the path.
					const uint8 UpdatePointIndex =  ShortPath.bPartialResult ? (ShortPath.NumPoints - ShortPath.NumPointsBeyondUpdate) : LastIndex;

					// If the shortpath is partial, it's expected to be full, meaning MaxPoints-NumPointsBeyondUpdate >= 1 so the check below must be valid.
					check(UpdatePointIndex >= 1);

					// Update entity progress on path (EntityDistanceToGoal)
					uint8 ClosestSegmentIndex = 0;
					float ClosestDistanceSquare = FLT_MAX;
					for (uint8 PointIndex = 0; PointIndex < UpdatePointIndex && PointIndex+1 <= LastIndex; PointIndex++)
					{
						// Project on closest segment
						const float DistanceSquared = FMath::PointDistToSegmentSquared(EntityLocation, ShortPath.Points[PointIndex].Position, ShortPath.Points[PointIndex+1].Position);
						if (DistanceSquared < ClosestDistanceSquare)
						{
							ClosestDistanceSquare = DistanceSquared;
							ClosestSegmentIndex = PointIndex;
						}
					}

					const FMassNavMeshPathPoint& PointA = ShortPath.Points[ClosestSegmentIndex];
					const FMassNavMeshPathPoint& PointB = ShortPath.Points[ClosestSegmentIndex+1];
					const float DistanceOnClosestSegment = FMath::Max(0.f, (EntityLocation - PointA.Position).Dot((PointB.Position - PointA.Position).GetSafeNormal()));
					const float EntityProgress = ShortPath.Points[ClosestSegmentIndex].Distance.Get() + DistanceOnClosestSegment;
					MoveTarget.EntityDistanceToGoal = FMath::Max(0.f, ShortPath.Points[UpdatePointIndex].Distance.Get() - EntityProgress);
					
#if WITH_MASSGAMEPLAY_DEBUG
					UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Verbose, TEXT("Entity [%s]   ProgressDistance: %.2f, UpdatePointIndex: %i, EntityDistanceToGoal: %.2f, bPartialResult: %i"),
						*Entity.DebugGetDescription(), ShortPath.MoveTargetProgressDistance, UpdatePointIndex, MoveTarget.EntityDistanceToGoal, ShortPath.bPartialResult);

					if (bDisplayDebug)
					{
						// Display update point
						const FVector ZOffset(0,0,20);
						UE_VLOG_CIRCLE(this, LogMassNavMeshNavigation, Display, ShortPath.Points[UpdatePointIndex].Position + ZOffset,
							FVector(0,0,1), /*Radius*/10.f, FColor::Green, TEXT("%i"), UpdatePointIndex);

						// Display entity progress
						const float DistanceAB = FMath::Max(PointB.Distance.Get() - PointA.Distance.Get(), KINDA_SMALL_NUMBER);
						const float T = (EntityProgress - ShortPath.Points[ClosestSegmentIndex].Distance.Get()) / DistanceAB;
						const FVector ProjectedEntityPosition = FMath::Lerp(PointA.Position, PointB.Position, T);
						UE_VLOG_SEGMENT(this, LogMassNavMeshNavigation, Display, EntityLocation + ZOffset, ProjectedEntityPosition + ZOffset, FColor::Silver, TEXT(""));
						UE_VLOG_CIRCLE(this, LogMassNavMeshNavigation, Display, ProjectedEntityPosition + ZOffset,
							FVector(0,0,1), /*Radius*/10.f, FColor::Silver, TEXT(""));
					}
#endif // WITH_MASSGAMEPLAY_DEBUG

					if (ShortPath.MoveTargetProgressDistance <= 0.0f)
					{
						// Before the start of the path
						MoveTarget.Center = ShortPath.Points[0].Position;
						MoveTarget.Forward = ShortPath.Points[0].Tangent.GetVector();
						MoveTarget.DistanceToGoal = ShortPath.Points[UpdatePointIndex].Distance.Get();

						UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Log,
							TEXT("Entity [%s]   before start of path. EntityDistanceToGoal: %.1f, DistanceToGoal: %.1f."),
							*Entity.DebugGetDescription(),
							MoveTarget.EntityDistanceToGoal,
							MoveTarget.DistanceToGoal);
					}
					else if (ShortPath.MoveTargetProgressDistance <= ShortPath.Points[UpdatePointIndex].Distance.Get())
					{
						// Along the path, interpolate.
						uint8 PointIndex = 0;
						while (PointIndex < (ShortPath.NumPoints - 2))
						{
							const FMassNavMeshPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];
							if (ShortPath.MoveTargetProgressDistance <= NextPoint.Distance.Get())
							{
								break;
							}
							PointIndex++;
						}
						const FMassNavMeshPathPoint& CurrPoint = ShortPath.Points[PointIndex];
						const FMassNavMeshPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];

						const float SegmentDistanceDelta = FMath::Max(NextPoint.Distance.Get() - CurrPoint.Distance.Get(), KINDA_SMALL_NUMBER);
						const float T = (ShortPath.MoveTargetProgressDistance - CurrPoint.Distance.Get()) / SegmentDistanceDelta;

						// Set move target location using the new progress distance.
						MoveTarget.Center = FMath::Lerp(CurrPoint.Position, NextPoint.Position, T);
						MoveTarget.Forward = FMath::Lerp(CurrPoint.Tangent.GetVector(), NextPoint.Tangent.GetVector(), T).GetSafeNormal();
						MoveTarget.DistanceToGoal = ShortPath.Points[UpdatePointIndex].Distance.Get() - FMath::Lerp(CurrPoint.Distance.Get(), NextPoint.Distance.Get(), T);

						UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Verbose, TEXT("Entity [%s]   Distance to goal on short path: %.1f."),
							*Entity.DebugGetDescription(), MoveTarget.DistanceToGoal);

						if (!ShortPath.bPartialResult)
						{
							// Last section of the path, wait for the steering to complete the movement.
							UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Log,
								TEXT("Entity [%s]   Waiting to reach the end of path."), *Entity.DebugGetDescription());

							CheckEndOfPathReached(ShortPath, EntityLocation, UpdatePointIndex, Entity, bDisplayDebug);
						}
					}
					else
					{
						// Update point reached.
						MoveTarget.Center = ShortPath.Points[UpdatePointIndex].Position;
						MoveTarget.Forward = ShortPath.Points[UpdatePointIndex].Tangent.GetVector();
						MoveTarget.DistanceToGoal = 0.f;

						if (ShortPath.bPartialResult)
						{
							UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Log,
								TEXT("Entity [%s]   Finished path follow on short path."), *Entity.DebugGetDescription());

							ShortPath.bDone = true;
						}
						else
						{
							// Last section of the path, wait for the steering to complete the movement.
							UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Log,
								TEXT("Entity [%s]   Waiting to reach the end of path."), *Entity.DebugGetDescription());
							
							CheckEndOfPathReached(ShortPath, EntityLocation, UpdatePointIndex, Entity, bDisplayDebug);
						}
					}
				}

				const bool bIsDone = ShortPath.IsDone();

				// Signal path done.
				if (!bWasDone && bIsDone)
				{
					UE_CVLOG_UELOG(bDisplayDebug, this, LogMassNavMeshNavigation, Log, TEXT("Entity [%s]   Signaling 'path done'."), *Entity.DebugGetDescription());
					EntitiesToSignalPathDone.Add(Entity);
				}

#if WITH_MASSGAMEPLAY_DEBUG
				if (bDisplayDebug)
				{
					const FColor EntityColor = UE::Mass::Debug::GetEntityDebugColor(Entity);

					const FVector ZOffset(0,0,10);
					FLinearColor MixColor(EntityColor);
					MixColor += FColor::White;
					MixColor /= 2;
					const FColor LightEntityColor = MixColor.ToFColorSRGB();

					FLinearColor BorderMixColor(EntityColor);
					BorderMixColor += FColor::Black;
					BorderMixColor /= 2;
					const FColor DarkEntityColor = BorderMixColor.ToFColorSRGB();

					// Draw path
					for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints - 1; PointIndex++)
					{
						const FMassNavMeshPathPoint& CurrPoint = ShortPath.Points[PointIndex];
						const FMassNavMeshPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];
						UE_VLOG_SEGMENT_THICK(this, LogMassNavMeshNavigation, Display, CurrPoint.Position + ZOffset, NextPoint.Position + ZOffset, EntityColor, /*Thickness*/3, TEXT(""));
						UE_VLOG_SEGMENT_THICK(this, LogMassNavMeshNavigation, Display, CurrPoint.Left + ZOffset, NextPoint.Left + ZOffset, DarkEntityColor, /*Thickness*/2, TEXT(""));
						UE_VLOG_SEGMENT_THICK(this, LogMassNavMeshNavigation, Display, CurrPoint.Right + ZOffset, NextPoint.Right + ZOffset, DarkEntityColor, /*Thickness*/2, TEXT(""));
					}

					// Draw point indices
					for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints; PointIndex++)
					{
						const FMassNavMeshPathPoint& CurrPoint = ShortPath.Points[PointIndex];
						UE_VLOG_CIRCLE(this, LogMassNavMeshNavigation, Display, CurrPoint.Position + ZOffset, FVector(0,0,1), /*Radius*/4.f, EntityColor, TEXT("%i"), PointIndex);
					}

					// Draw tangents
					for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints; PointIndex++)
					{
						const FMassNavMeshPathPoint& CurrPoint = ShortPath.Points[PointIndex];
						const FVector CurrBase = CurrPoint.Position + ZOffset + FVector(0,0,1);
						UE_VLOG_SEGMENT_THICK(this, LogMassNavMeshNavigation, Verbose, CurrBase, CurrBase + CurrPoint.Tangent.GetVector() * 100.0f, LightEntityColor, /*Thickness*/1, TEXT(""));
					}

					// Draw MoveTarget
					constexpr float Radius = 20.f;
					UE_VLOG_CIRCLE(this, LogMassNavMeshNavigation, Display, MoveTarget.Center + ZOffset, FVector(0,0,1), Radius, LightEntityColor,
						TEXT("IntentAtGoal: %s"), *UEnum::GetDisplayValueAsText(MoveTarget.IntentAtGoal).ToString());

					// Draw tolerance distance at end (shown as a circle instead of an infinite perpendicular line)
					if (!ShortPath.bPartialResult)
					{
						const float EndRadius = ShortPath.EndReachedDistance;
						UE_VLOG_WIRECIRCLE(this, LogMassNavMeshNavigation, Display, ShortPath.Points[ShortPath.NumPoints-1].Position + ZOffset,
							FVector(0,0,1), EndRadius, FColor::Black, TEXT("End"));
					}
				}
#endif // WITH_MASSGAMEPLAY_DEBUG
			}
		}
	});

	if (EntitiesToSignalPathDone.Num())
	{
		SignalSubsystem->SignalEntities(UE::Mass::Signals::FollowPointPathDone, EntitiesToSignalPathDone);
	}
}
