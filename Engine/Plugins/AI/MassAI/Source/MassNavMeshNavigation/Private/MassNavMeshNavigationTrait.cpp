// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavMeshNavigationTrait.h"

#include "MassCommonFragments.h"
#include "MassEntityTemplateRegistry.h"
#include "MassNavigationFragments.h"
#include "MassNavMeshNavigationFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavMeshNavigationTrait)

void UMassNavMeshNavigationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.RequireFragment<FMassMoveTargetFragment>();

	BuildContext.AddFragment<FAgentRadiusFragment>();
	BuildContext.AddFragment<FAgentHeightFragment>();

	BuildContext.AddFragment<FMassNavMeshCachedPathFragment>();
	BuildContext.AddFragment<FMassNavMeshShortPathFragment>();

	BuildContext.AddFragment<FMassNavMeshBoundaryFragment>();	
}
