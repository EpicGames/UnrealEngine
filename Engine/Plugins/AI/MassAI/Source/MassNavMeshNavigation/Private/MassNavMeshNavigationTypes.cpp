// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavMeshNavigationTypes.h"

DEFINE_LOG_CATEGORY(LogMassNavMeshNavigation);
