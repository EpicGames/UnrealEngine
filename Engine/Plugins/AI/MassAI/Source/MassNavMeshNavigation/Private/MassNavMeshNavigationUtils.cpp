// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavMeshNavigationUtils.h"
#include "MassCommonTypes.h"
#include "MassNavigationFragments.h"
#include "MassNavMeshNavigationFragments.h"
#include "VisualLogger/VisualLogger.h"

namespace UE::MassNavigation
{
	bool ActivateActionStand(const UObject* Requester,
							 const FMassEntityHandle Entity,
							 const float DesiredSpeed,
							 FMassMoveTargetFragment& InOutMoveTarget,
							 FMassNavMeshShortPathFragment& OutShortPath)
	{
		OutShortPath.Reset();
		InOutMoveTarget.DistanceToGoal = 0.f;
		InOutMoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
		InOutMoveTarget.DesiredSpeed.Set(0.f);
		
		if (!ensureMsgf(InOutMoveTarget.GetCurrentAction() == EMassMovementAction::Stand,
				TEXT("Expecting action 'Stand': Invalid action %s."), *UEnum::GetDisplayValueAsText(InOutMoveTarget.GetCurrentAction()).ToString()))
		{
			return false;
		}
		
		InOutMoveTarget.IntentAtGoal = EMassMovementAction::Stand;
		InOutMoveTarget.DesiredSpeed.Set(DesiredSpeed);
		
		UE_VLOG_UELOG(Requester, LogMassNavigation, Log, TEXT("Entity [%s] successfully requested %s."), *Entity.DebugGetDescription(), *InOutMoveTarget.ToString());

		return true;
	}

	bool ActivateActionAnimate(const UObject* Requester,
						       const FMassEntityHandle Entity,
						       FMassMoveTargetFragment& MoveTarget)
	{
		MoveTarget.DistanceToGoal = 0.f;
		MoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
		MoveTarget.DesiredSpeed.Set(0.f);

		if (!ensureMsgf(MoveTarget.GetCurrentAction() == EMassMovementAction::Animate, TEXT("Expecting action 'Animate': Invalid action %u"), EnumToUnderlyingType(MoveTarget.GetCurrentAction())))
		{
			return false;
		}

		MoveTarget.IntentAtGoal = EMassMovementAction::Stand;

		UE_VLOG_UELOG(Requester, LogMassNavigation, Log, TEXT("Entity [%s] successfully requested %s"), *Entity.DebugGetDescription(), *MoveTarget.ToString());

		return true;
	}
}
