// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "MassNavMeshNavigationBoundaryProcessor.generated.h"

#define UE_API MASSNAVMESHNAVIGATION_API

/** Fills FMassNavigationEdgesFragment using FMassNavMeshShortPathFragment. */
UCLASS(MinimalAPI)
class UMassNavMeshNavigationBoundaryProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassNavMeshNavigationBoundaryProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};

#undef UE_API
