// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "MassCommonTypes.h"
#include "MassNavigationTypes.h"
#include "AI/Navigation/NavigationTypes.h"
#include "Containers/StaticArray.h"
#include "MassNavMeshNavigationFragments.generated.h"

#define UE_API MASSNAVMESHNAVIGATION_API

struct FNavCorridor;

/** Navmesh path point data */
USTRUCT()
struct FMassNavMeshPathPoint
{
	GENERATED_BODY()

	/** Portal left limit. */
	FVector Left = FVector::ZeroVector;
	
	/** Portal right limit. */
	FVector Right = FVector::ZeroVector;

	/** Position of the path. */
	FVector Position = FVector::ZeroVector;

	/** Tangent direction of the path. */
	FMassSnorm8Vector2D Tangent;

	/** Distance along the path from first point. */
	FMassInt16Real Distance = FMassInt16Real(0.f);
};

/** Short path used for navmesh navigation */
USTRUCT()
struct FMassNavMeshShortPathFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassNavMeshShortPathFragment() = default;

	/* Maximum number of points on the short path */
	static constexpr uint8 MaxPoints = 8;

	/* Number of points beyond the update point */
	static constexpr uint8 NumPointsBeyondUpdate = 3;

	void Reset()
	{
		MoveTargetProgressDistance = 0.f;
		EndReachedDistance = 20.f;
		NumPoints = 0;
		EndOfPathIntent = EMassMovementAction::Stand;
		bPartialResult = false;
		bDone = false;
		bInitialized = false;
	}

	/** Returns true if we are done progressing on the short path. */
	bool IsDone() const
	{
		return NumPoints == 0 || bDone;
	}

	/** Fill Points from Corridor. */
	UE_API bool RequestShortPath(const TSharedPtr<FNavCorridor>& InCorridor, const int32 InNavCorridorStartIndex, const uint8 InNumLeadingPoints, const float InEndReachedDistance);

	/** Portal points */
	TStaticArray<FMassNavMeshPathPoint, MaxPoints> Points;
	
	/** Current progress distance along the short path. */
	float MoveTargetProgressDistance = 0.f;

	/** Distance from the end of path used to confirm that the destination is reached. */
	float EndReachedDistance = 20.f;
	
	/** Number of points on path. */
	uint8 NumPoints = 0;
	
	/** Intent at the end of the path. */
	EMassMovementAction EndOfPathIntent = EMassMovementAction::Stand;

	/** True if the path was partial. */
	uint8 bPartialResult : 1 = false;

	/** True when path follow is completed. */
	uint8 bDone : 1 = false;

	/** True when the path has been initalized. */
	uint8 bInitialized : 1 = false;
};

enum class EMassNavigationPathSource : uint8
{
	NavMesh,
	Spline,
	Unset
};

/** Current navmesh path */
USTRUCT()
struct FMassNavMeshCachedPathFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Reference to a FNavigationPath. */
	FNavPathSharedPtr NavPath;

	/** Reference to an FNavCorridor. Built out of a navigation path. */
	TSharedPtr<FNavCorridor> Corridor;

	/** Index used to keep track of progression on the navmesh path. */
	uint16 NavPathNextStartIndex = 0;

	/** Source of the path, navmesh or else. */
	EMassNavigationPathSource PathSource = EMassNavigationPathSource::Unset;

	/** Number of points before the starting point. */
	static constexpr uint8 NumLeadingPoints = 1;
};

template<>
struct TMassFragmentTraits<FMassNavMeshCachedPathFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

USTRUCT()
struct FMassNavMeshBoundaryFragment : public FMassFragment
{
	GENERATED_BODY()

	/** MovementTarget position when UMassNavMeshNavigationBoundaryProcessor was last updated. Used to identify when a new update is needed. */
	FVector LastUpdatePosition = FVector::ZeroVector;
};

#undef UE_API
