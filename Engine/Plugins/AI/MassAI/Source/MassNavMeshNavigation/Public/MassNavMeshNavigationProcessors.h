// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "MassNavMeshNavigationProcessors.generated.h"

#define UE_API MASSNAVMESHNAVIGATION_API

class UMassSignalSubsystem;
struct FMassNavMeshShortPathFragment;

/** Processor for updating move target on a navmesh short path. */
UCLASS(MinimalAPI)
class UMassNavMeshPathFollowProcessor : public UMassProcessor
{
	GENERATED_BODY()

protected:
	UE_API UMassNavMeshPathFollowProcessor();

	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void InitializeInternal(UObject& Owner, const TSharedRef<FMassEntityManager>& ) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	// Check if entity reached end of path and mark as done if threshold is met.
	void CheckEndOfPathReached(FMassNavMeshShortPathFragment& ShortPath,
		const FVector& EntityLocation, const uint8 EndPointIndex, const FMassEntityHandle& Entity, bool bDisplayDebug);

	FMassEntityQuery EntityQuery_Conditional;

	UPROPERTY(Transient)
	TObjectPtr<UMassSignalSubsystem> SignalSubsystem = nullptr;
};

#undef UE_API
