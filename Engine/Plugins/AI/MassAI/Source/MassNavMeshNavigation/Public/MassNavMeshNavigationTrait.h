// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassNavMeshNavigationTrait.generated.h"

#define UE_API MASSNAVMESHNAVIGATION_API

UCLASS(MinimalAPI, meta = (DisplayName = "NavMesh Navigation"))
class UMassNavMeshNavigationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;
};

#undef UE_API
