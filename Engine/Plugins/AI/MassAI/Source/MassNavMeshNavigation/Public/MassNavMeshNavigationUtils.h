// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EngineDefines.h"

class UObject;
struct FMassEntityHandle;
struct FMassMoveTargetFragment;
struct FMassNavMeshShortPathFragment;

namespace UE::MassNavigation
{
	MASSNAVMESHNAVIGATION_API bool ActivateActionStand(
		const UObject* Requester, const FMassEntityHandle Entity, const float DesiredSpeed, FMassMoveTargetFragment& InOutMoveTarget, FMassNavMeshShortPathFragment& OutShortPath);

	MASSNAVMESHNAVIGATION_API bool ActivateActionAnimate(
		const UObject* Requester, const FMassEntityHandle Entity, FMassMoveTargetFragment& MoveTarget);
};
