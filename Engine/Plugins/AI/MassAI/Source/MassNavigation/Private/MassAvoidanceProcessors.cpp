// Copyright Epic Games, Inc. All Rights Reserved.

#include "Avoidance/MassAvoidanceProcessors.h"
#include "Avoidance/MassAvoidanceFragments.h"
#include "DrawDebugHelpers.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "VisualLogger/VisualLogger.h"
#include "Math/Vector2D.h"
#include "Logging/LogMacros.h"
#include "MassSimulationLOD.h"
#include "MassCommonFragments.h"
#include "MassDebugLogging.h"
#include "MassMovementFragments.h"
#include "MassNavigationSubsystem.h"
#include "MassNavigationFragments.h"
#include "MassNavigationUtils.h"
#include "Engine/World.h"
#include "MassDebugger.h"
#include "MassNavigationDebug.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassAvoidanceProcessors)

DEFINE_LOG_CATEGORY(LogAvoidance);
DEFINE_LOG_CATEGORY(LogAvoidanceVelocities);
DEFINE_LOG_CATEGORY(LogAvoidanceAgents);
DEFINE_LOG_CATEGORY(LogAvoidanceObstacles);

namespace UE::MassAvoidance
{
	namespace Tweakables
	{
		bool bEnableEnvironmentAvoidance = true;
		bool bEnableSettingsForExtendingColliders = true;
		bool bUseAdjacentCorridors = true;
		bool bUseDrawDebugHelpers = false;
		bool bEnableDetailedDebug = false;
	} // Tweakables

	FAutoConsoleVariableRef Vars[] = 
	{
		FAutoConsoleVariableRef(TEXT("ai.mass.avoidance.EnableEnvironmentAvoidance"), Tweakables::bEnableEnvironmentAvoidance, TEXT("Set to false to disable avoidance forces for environment (for debug purposes)."), ECVF_Cheat),
		FAutoConsoleVariableRef(TEXT("ai.mass.avoidance.EnableSettingsForExtendingColliders"), Tweakables::bEnableSettingsForExtendingColliders, TEXT("Set to false to disable using different settings for extending obstacles (for debug purposes)."), ECVF_Cheat),
		FAutoConsoleVariableRef(TEXT("ai.mass.avoidance.UseAdjacentCorridors"), Tweakables::bUseAdjacentCorridors, TEXT("Set to false to disable usage of adjacent lane width."), ECVF_Cheat),
		FAutoConsoleVariableRef(TEXT("ai.mass.avoidance.UseDrawDebugHelpers"), Tweakables::bUseDrawDebugHelpers, TEXT("Use debug draw helpers in addition to visual logs."), ECVF_Cheat),
		FAutoConsoleVariableRef(TEXT("ai.mass.avoidance.EnableDetailedDebug"), Tweakables::bEnableDetailedDebug, TEXT("Display additional avoidance debug information."), ECVF_Cheat)
	};

	constexpr int32 MaxExpectedAgentsPerCell = 6;
	constexpr int32 MinTouchingCellCount = 4;
	constexpr int32 MaxObstacleResults = MaxExpectedAgentsPerCell * MinTouchingCellCount;

	static void FindCloseObstacles(const FVector& Center, const FVector::FReal SearchRadius, const FNavigationObstacleHashGrid2D& AvoidanceObstacleGrid,
									TArray<FMassNavigationObstacleItem, TFixedAllocator<MaxObstacleResults>>& OutCloseEntities, const int32 MaxResults)
	{
		OutCloseEntities.Reset();
		const FVector Extent(SearchRadius, SearchRadius, 0.);
		const FBox QueryBox = FBox(Center - Extent, Center + Extent);

		struct FSortingCell
		{
			int32 X;
			int32 Y;
			int32 Level;
			FVector::FReal SqDist;
		};
		TArray<FSortingCell, TInlineAllocator<64>> Cells;
		const FVector QueryCenter = QueryBox.GetCenter();
		
		for (int32 Level = 0; Level < AvoidanceObstacleGrid.NumLevels; Level++)
		{
			const FVector::FReal CellSize = AvoidanceObstacleGrid.GetCellSize(Level);
			const FNavigationObstacleHashGrid2D::FCellRect Rect = AvoidanceObstacleGrid.CalcQueryBounds(QueryBox, Level);

			// Use int64 to prevent overflow when MaxX or MaxY are equal to the maximum value of int32.
			for (int64 Y = Rect.MinY; Y <= Rect.MaxY; Y++)
			{
				for (int64 X = Rect.MinX; X <= Rect.MaxX; X++)
				{
					const FVector::FReal CenterX = (X + 0.5) * CellSize;
					const FVector::FReal CenterY = (Y + 0.5) * CellSize;
					const FVector::FReal DX = CenterX - QueryCenter.X;
					const FVector::FReal DY = CenterY - QueryCenter.Y;
					const FVector::FReal SqDist = DX * DX + DY * DY;
					FSortingCell SortCell;
					SortCell.X = X;
					SortCell.Y = Y;
					SortCell.Level = Level;
					SortCell.SqDist = SqDist;
					Cells.Add(SortCell);
				}
			}
		}

		Cells.Sort([](const FSortingCell& A, const FSortingCell& B) { return A.SqDist < B.SqDist; });

		for (const FSortingCell& SortedCell : Cells)
		{
			if (const FNavigationObstacleHashGrid2D::FCell* Cell = AvoidanceObstacleGrid.FindCell(SortedCell.X, SortedCell.Y, SortedCell.Level))
			{
				const TSparseArray<FNavigationObstacleHashGrid2D::FItem>&  Items = AvoidanceObstacleGrid.GetItems();
				for (int32 Idx = Cell->First; Idx != INDEX_NONE; Idx = Items[Idx].Next)
				{
					OutCloseEntities.Add(Items[Idx].ID);
					if (OutCloseEntities.Num() >= MaxResults)
					{
						return;
					}
				}
			}
		}
	}

	// Adapted from ray-capsule intersection: https://iquilezles.org/www/articles/intersectors/intersectors.htm
	static FVector::FReal ComputeClosestPointOfApproach(const FVector2D Pos, const FVector2D Vel, const FVector::FReal Rad, const FVector2D SegStart, const FVector2D SegEnd, const FVector::FReal TimeHoriz)
	{
		const FVector2D SegDir = SegEnd - SegStart;
		const FVector2D RelPos = Pos - SegStart;
		const FVector::FReal VelSq = FVector2D::DotProduct(Vel, Vel);
		const FVector::FReal SegDirSq = FVector2D::DotProduct(SegDir, SegDir);
		const FVector::FReal DirVelSq = FVector2D::DotProduct(SegDir, Vel);
		const FVector::FReal DirRelPosSq = FVector2D::DotProduct(SegDir, RelPos);
		const FVector::FReal VelRelPosSq = FVector2D::DotProduct(Vel, RelPos);
		const FVector::FReal RelPosSq = FVector2D::DotProduct(RelPos, RelPos);
		const FVector::FReal A = SegDirSq * VelSq - DirVelSq * DirVelSq;
		const FVector::FReal B = SegDirSq * VelRelPosSq - DirRelPosSq * DirVelSq;
		const FVector::FReal C = SegDirSq * RelPosSq - DirRelPosSq * DirRelPosSq - FMath::Square(Rad) * SegDirSq;
		const FVector::FReal H = FMath::Max(0., B*B - A*C); // b^2 - ac, Using max for closest point of arrival result when no hit.
		const FVector::FReal T = FMath::Abs(A) > SMALL_NUMBER ? (-B - FMath::Sqrt(H)) / A : 0.;
		const FVector::FReal Y = DirRelPosSq + T * DirVelSq;
		
		if (Y > 0. && Y < SegDirSq) 
		{
			return FMath::Clamp(T, 0., TimeHoriz);
		}
		else 
		{
			// caps
			const FVector2D CapRelPos = (Y <= 0.) ? RelPos : Pos - SegEnd;
			const FVector::FReal Cb = FVector2D::DotProduct(Vel, CapRelPos);
			const FVector::FReal Cc = FVector2D::DotProduct(CapRelPos, CapRelPos) - FMath::Square(Rad);
			const FVector::FReal Ch = FMath::Max(0., Cb * Cb - VelSq * Cc);
			const FVector::FReal T1 = VelSq > SMALL_NUMBER ? (-Cb - FMath::Sqrt(Ch)) / VelSq : 0.;
			return FMath::Clamp(T1, 0., TimeHoriz);
		}
	}

	static FVector::FReal ComputeClosestPointOfApproach(const FVector RelPos, const FVector RelVel, const FVector::FReal TotalRadius, const FVector::FReal TimeHoriz)
	{
		// Calculate time of impact based on relative agent positions and velocities.
		const FVector::FReal A = FVector::DotProduct(RelVel, RelVel);
		const FVector::FReal Inv2A = A > SMALL_NUMBER ? 1. / (2. * A) : 0.;
		const FVector::FReal B = FMath::Min(0., 2. * FVector::DotProduct(RelVel, RelPos));
		const FVector::FReal C = FVector::DotProduct(RelPos, RelPos) - FMath::Square(TotalRadius);
		// Using max() here gives us CPA (closest point on arrival) when there is no hit.
		const FVector::FReal Discr = FMath::Sqrt(FMath::Max(0., B * B - 4. * A * C));
		const FVector::FReal T = (-B - Discr) * Inv2A;
		return FMath::Clamp(T, 0., TimeHoriz);
	}

#if WITH_MASSGAMEPLAY_DEBUG
	using namespace UE::MassNavigation::Debug;
	
	// Colors
	static constexpr FColor Amber(255,179,0);
	static constexpr FColor Orange(251,140,0);
	static constexpr FColor OrangeRed(244,81,30);
	static constexpr FColor Cyan(0,172,193);
	static constexpr FColor Blue(2,155,229);
	static constexpr FColor Indigo(57,73,171);
	static constexpr FColor Yellow(253, 216, 53);
	static constexpr FColor Teal(0, 137, 123);
	static constexpr FColor Lime(172, 222, 51);

	static constexpr FColor CurrentAgentColor = Lime;
	static const	 FColor VelocityColor = FColor::Black;
	static constexpr FColor DesiredVelocityColor = Yellow;
	static constexpr FColor FinalSteeringForceColor = Teal;

	// Agents colors
	static constexpr FColor AgentsColor = Amber;
	static constexpr FColor AgentsNotMovingColor = Blue;
	static constexpr FColor AgentAvoidForceColor = Orange;
	static constexpr FColor AgentsNotMovingSeparationColor = Blue;
	static constexpr FColor AgentSeparationForceColor = OrangeRed;
	
	// Obstacles colors
	static constexpr FColor ObstacleColor = Cyan;	// edges
	static constexpr FColor ObstacleAvoidForceColor = Blue;
	static constexpr FColor ObstacleSeparationForceColor = Indigo;
	static const	FColor ObstacleContactNormalColor = FColor::Silver;

	// Ghost colors
	static const FColor GhostColor = FColor::Silver;
	static const FColor GhostSteeringForceColor = FColor::Silver;

	// Forces thickness
	static constexpr float AvoidThickness = 3.f;
	static constexpr float SeparationThickness = 3.f;
	static constexpr float SummedForcesThickness = 5.f;

	// Output force
	static constexpr float SteeringThickness = 8.f;
	static constexpr float SteeringArrowHeadSize = 12.f;

	// Height offsets
	static const FVector DebugAgentHeightOffset = FVector(0., 0., 175.);
	static const FVector DebugInputForceHeight = FVector(0., 0., 181.);
	static const FVector DebugAgentAvoidHeightOffset = FVector(0., 0., 182.);
	static const FVector DebugAgentSeparationHeightOffset = FVector(0., 0., 183.);
	static const FVector DebugOutputForcesHeight = FVector(0., 0., 184.);
	static const FVector DebugLowCylinderOffset = FVector(0., 0., 20.);
	
	// Local debug utils
	static void DebugDrawVelocity(const FDebugContext& Context, const FVector& Start, const FVector& End, const FColor& Color)
	{
		// Different arrow than DebugDrawArrow()
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		constexpr float Thickness = 3.f;
		constexpr FVector::FReal Pointyness = 1.8;
		const FVector Line = End - Start;
		const FVector UnitV = Line.GetSafeNormal();
		const FVector Perp = FVector::CrossProduct(UnitV, FVector::UpVector);
		const FVector Left = Perp - (Pointyness * UnitV);
		const FVector Right = -Perp - (Pointyness * UnitV);
		const FVector::FReal HeadSize = 0.08 * Line.Size();
		const UObject* LogOwner = Context.GetLogOwner(); 
		UE_VLOG_SEGMENT_THICK(LogOwner, Context.Category, Log, Start, End, Color, (int16)Thickness, TEXT(""));
		UE_VLOG_SEGMENT_THICK(LogOwner, Context.Category, Log, End, End + HeadSize * Left, Color, (int16)Thickness, TEXT(""));
		UE_VLOG_SEGMENT_THICK(LogOwner, Context.Category, Log, End, End + HeadSize * Right, Color, (int16)Thickness, TEXT(""));
		UE_VLOG_SEGMENT_THICK(LogOwner, Context.Category, Log, End + HeadSize * Left, End + HeadSize * Right, Color, (int16)Thickness, TEXT(""));

		if (UseDrawDebugHelper() && Context.World)
		{
			DrawDebugLine(Context.World, Start, End, Color, /*bPersistent=*/ false, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
			DrawDebugLine(Context.World, End, End + HeadSize * Left, Color, /*bPersistent=*/ false, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
			DrawDebugLine(Context.World, End, End + HeadSize * Right, Color, /*bPersistent=*/ false, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
			DrawDebugLine(Context.World, End + HeadSize * Left, End + HeadSize * Right, Color, /*bPersistent=*/ false, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
		}
	}

	static void DebugDrawForce(const FDebugContext& Context, const FVector& Start, const FVector& End, const FColor& Color, const float Thickness, const FString& Text = FString())
	{
		DebugDrawArrow(Context, Start, End, Color, /*HeadSize*/4.f, Thickness, Text);
	}

	static void DebugDrawSummedForce(const FDebugContext& Context, const FVector& Start, const FVector& End, const FColor& Color)
	{
		DebugDrawArrow(Context, Start + FVector(0.,0.,1.), End + FVector(0., 0., 1.), Color, /*HeadSize*/8.f, SummedForcesThickness);
	}

	static void DebugDrawHitPosition(const FDebugContext& Context, const FVector& Location, const FVector& Velocity,
		const FVector::FReal CPA, const FVector::FReal Radius, const FColor Color)
	{
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		const FVector HitPosition = Location + (Velocity * CPA);
		const FVector LeftOffset = Radius * UE::MassNavigation::GetLeftDirection(Velocity.GetSafeNormal(), FVector::UpVector);
		UE::MassAvoidance::DebugDrawLine(Context, Location + UE::MassAvoidance::DebugAgentHeightOffset + LeftOffset,
			HitPosition + UE::MassAvoidance::DebugAgentHeightOffset + LeftOffset, Color, /*Thickness*/1.5f);
		UE::MassAvoidance::DebugDrawLine(Context, Location + UE::MassAvoidance::DebugAgentHeightOffset - LeftOffset,
			HitPosition + UE::MassAvoidance::DebugAgentHeightOffset - LeftOffset, Color, /*Thickness*/1.5f);
		UE::MassAvoidance::DebugDrawCylinder(Context, HitPosition,
			HitPosition + UE::MassAvoidance::DebugAgentHeightOffset, Radius, Color);
	}

#endif // WITH_MASSGAMEPLAY_DEBUG

} // namespace UE::MassAvoidance


//----------------------------------------------------------------------//
//  UMassMovingAvoidanceProcessor
//----------------------------------------------------------------------//
UMassMovingAvoidanceProcessor::UMassMovingAvoidanceProcessor()
	: EntityQuery(*this) 
{
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Avoidance;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);
}

void UMassMovingAvoidanceProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassForceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadWrite, EMassFragmentPresence::Optional);
	EntityQuery.AddRequirement<FMassNavigationEdgesFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassAvoidanceEntitiesToIgnoreFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.AddTagRequirement<FMassMediumLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassLowLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassAvoidancePreventBackwardMovementTag>(EMassFragmentPresence::Optional);
	EntityQuery.AddConstSharedRequirement<FMassMovingAvoidanceParameters>(EMassFragmentPresence::All);
	EntityQuery.AddConstSharedRequirement<FMassMovementParameters>(EMassFragmentPresence::All);
	EntityQuery.AddSubsystemRequirement<UMassNavigationSubsystem>(EMassFragmentAccess::ReadOnly);

	// Indirect access: fragments read on neighbor entities returned by FindCloseObstacles
	EntityQuery.AddIndirectFragmentRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddIndirectFragmentRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddIndirectFragmentRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddIndirectFragmentRequirement<FMassAvoidanceColliderFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddIndirectFragmentRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly);

#if WITH_MASSGAMEPLAY_DEBUG
	EntityQuery.DebugEnableEntityOwnerLogging();
#endif
}

void UMassMovingAvoidanceProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UMassMovingAvoidanceProcessor);

	UWorld* DebugDrawWorld = UE::Mass::Debug::GetDebugDrawWorld(Context);
	UWorld* World = Context.GetWorld();
	check(World);
	const double CurrentTime = World->GetTimeSeconds();
	const float DeltaTime = Context.GetDeltaTimeSeconds();

	EntityQuery.ForEachEntityChunk(Context, [FallbackLogOwner = this, &EntityManager, DebugDrawWorld, CurrentTime, DeltaTime](FMassExecutionContext& Context)
	{
		const UMassNavigationSubsystem* NavigationSubsystem = Context.GetSubsystem<UMassNavigationSubsystem>();
		if (!NavigationSubsystem)
		{
			return;
		}
		const TArrayView<FMassForceFragment> ForceList = Context.GetMutableFragmentView<FMassForceFragment>();
		const TArrayView<FMassDesiredMovementFragment> DesiredMovementList = Context.GetMutableFragmentView<FMassDesiredMovementFragment>();
		const bool bHasDesiredMovementFragments = !DesiredMovementList.IsEmpty();
		const TConstArrayView<FMassNavigationEdgesFragment> NavEdgesList = Context.GetFragmentView<FMassNavigationEdgesFragment>();
		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();
		const TConstArrayView<FAgentRadiusFragment> RadiusList = Context.GetFragmentView<FAgentRadiusFragment>();
		const TConstArrayView<FMassAvoidanceEntitiesToIgnoreFragment> EntitiesToIgnoreList = Context.GetFragmentView<FMassAvoidanceEntitiesToIgnoreFragment>();
		const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();
		const FMassMovingAvoidanceParameters& MovingAvoidanceParams = Context.GetConstSharedFragment<FMassMovingAvoidanceParameters>();
		const FMassMovementParameters& MovementParams = Context.GetConstSharedFragment<FMassMovementParameters>();
		const bool bPreventBackwardMovement = Context.DoesArchetypeHaveTag<FMassAvoidancePreventBackwardMovementTag>();

		const FVector::FReal InvPredictiveAvoidanceTime = 1. / MovingAvoidanceParams.PredictiveAvoidanceTime;
		const FVector::FReal InvEnvironmentPredictiveAvoidanceTime = 1. / MovingAvoidanceParams.EnvironmentPredictiveAvoidanceTime;

		// Arrays used to store close obstacles
		TArray<FMassNavigationObstacleItem, TFixedAllocator<UE::MassAvoidance::MaxObstacleResults>> CloseEntities;

		// Used for storing sorted list or nearest obstacles.
		struct FSortedObstacle
		{
			FVector LocationCached;
			FVector Forward;
			FMassNavigationObstacleItem ObstacleItem;
			FVector::FReal SqDist;
		};
		TArray<FSortedObstacle, TFixedAllocator<UE::MassAvoidance::MaxObstacleResults>> ClosestObstacles;

		// Potential contact between agent and environment. 
		struct FEnvironmentContact
		{
			FVector Position = FVector::ZeroVector;
			FVector Normal = FVector::ZeroVector;
			FVector::FReal Distance = 0.;				// Distance from agent center to contact position
			bool bAlongTheEdge = false;
			bool bBehindTheEdge = false;
		};
		TArray<FEnvironmentContact, TInlineAllocator<16>> Contacts;

		// Describes collider to avoid, collected from neighbour obstacles.
		struct FCollider
		{
			FVector Location = FVector::ZeroVector;
			FVector Velocity = FVector::ZeroVector;
			float Radius = 0.f;
			bool bCanAvoid = true;
			bool bIsMoving = false;
		};
		TArray<FCollider, TInlineAllocator<16>> Colliders;

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// @todo: this check should eventually be part of the query (i.e. only handle moving agents).
			const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
			if (MoveTarget.GetCurrentAction() == EMassMovementAction::Animate || MoveTarget.GetCurrentAction() == EMassMovementAction::Stand)
			{
				continue;
			}

			FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			const FMassNavigationEdgesFragment& NavEdges = NavEdgesList[EntityIt];
			const FTransformFragment& Location = LocationList[EntityIt];
			const FMassVelocityFragment& Velocity = VelocityList[EntityIt];
			const FAgentRadiusFragment& RadiusFragment = RadiusList[EntityIt];
			
			TConstArrayView<FMassEntityHandle> EntitiesToIgnore =
				EntitiesToIgnoreList.IsEmpty() ?
				TConstArrayView<FMassEntityHandle>() :
					EntitiesToIgnoreList[EntityIt].EntitiesToIgnore;
			
			FMassForceFragment& Force = ForceList[EntityIt];

			// Smaller steering max accel makes the steering more "calm" but less opportunistic, may not find solution, or gets stuck.
			// Max contact accel should be quite a big bigger than steering so that collision response is firm. 
			const FVector::FReal MaxSteerAccel = MovementParams.MaxAcceleration;
			const FVector::FReal MaximumSpeed = MovementParams.MaxSpeed;

			const FVector AgentLocation = Location.GetTransform().GetTranslation();
			const FVector AgentVelocity = FVector(Velocity.Value.X, Velocity.Value.Y, 0.);
			
			const FVector::FReal AgentRadius = RadiusFragment.Radius;
			
			FVector SteeringForce = Force.Value;

			// Near start and end fades are used to subdue the avoidance at the start and end of the path.
			FVector::FReal NearStartFade = 1.;
			FVector::FReal NearEndFade = 1.;

			if (MoveTarget.GetPreviousAction() != EMassMovementAction::Move)
			{
				// Fade in avoidance when transitioning from other than move action.
				// I.e. the standing behavior may move the agents so close to each,
				// and that causes the separation to push them out quickly when avoidance is activated. 
				NearStartFade = FMath::Min((CurrentTime - MoveTarget.GetCurrentActionStartTime()) / MovingAvoidanceParams.StartOfPathDuration, 1.);
			}

			if (MoveTarget.IntentAtGoal == EMassMovementAction::Stand)
			{
				// Estimate approach based on current desired speed.
				const FVector::FReal ApproachDistance = FMath::Max<FVector::FReal>(1., MovingAvoidanceParams.EndOfPathDuration * MoveTarget.DesiredSpeed.Get());
				NearEndFade = FMath::Clamp(MoveTarget.DistanceToGoal / ApproachDistance, 0., 1.);
			}
			
			const FVector::FReal NearStartScaling = FMath::Lerp<FVector::FReal>(MovingAvoidanceParams.StartOfPathAvoidanceScale, 1., NearStartFade);
			const FVector::FReal NearEndScaling = FMath::Lerp<FVector::FReal>(MovingAvoidanceParams.EndOfPathAvoidanceScale, 1., NearEndFade);
			
#if WITH_MASSGAMEPLAY_DEBUG
			const UE::MassAvoidance::FDebugContext BaseDebugContext(Context, FallbackLogOwner, LogAvoidance, DebugDrawWorld, Entity, EntityIt);
			const UE::MassAvoidance::FDebugContext VelocitiesDebugContext(Context, FallbackLogOwner, LogAvoidanceVelocities, DebugDrawWorld, Entity, EntityIt);
			const UE::MassAvoidance::FDebugContext ObstacleDebugContext(Context, FallbackLogOwner, LogAvoidanceObstacles, DebugDrawWorld, Entity, EntityIt);
			const UE::MassAvoidance::FDebugContext AgentDebugContext(Context, FallbackLogOwner, LogAvoidanceAgents, DebugDrawWorld, Entity, EntityIt);

			FColor EntityColor = FColor::White;
			if (BaseDebugContext.ShouldLogEntity(&EntityColor))
			{
				// Draw agent
				const FString Text = FString::Printf(TEXT("%i"), Entity.Index);
				DebugDrawCylinder(BaseDebugContext, AgentLocation, AgentLocation + UE::MassAvoidance::DebugAgentHeightOffset, (AgentRadius+1.),
					UE::MassAvoidance::CurrentAgentColor, Text);

				// Draw agent center
				DebugDrawSphere(BaseDebugContext, AgentLocation, 10.f, UE::MassAvoidance::CurrentAgentColor);

				// Draw circle for agent in LogMassNavigation.
				const FVector ZOffset(0,0,25);
				UE_VLOG_WIRECIRCLE_THICK(FallbackLogOwner, LogMassNavigation, Log, AgentLocation + ZOffset, FVector::UpVector, AgentRadius, EntityColor, /*Thickness*/4,
					TEXT("%s"), *Entity.DebugGetDescription(), TEXT("%s"), *Entity.DebugGetDescription());

				// Draw current velocity (black)
				UE::MassAvoidance::DebugDrawVelocity(VelocitiesDebugContext, AgentLocation + UE::MassAvoidance::DebugInputForceHeight,
				                                     AgentLocation + UE::MassAvoidance::DebugInputForceHeight + AgentVelocity, UE::MassAvoidance::VelocityColor);

				// Draw initial steering force
				DebugDrawArrow(BaseDebugContext, AgentLocation + UE::MassAvoidance::DebugInputForceHeight,
					AgentLocation + UE::MassAvoidance:: DebugInputForceHeight + SteeringForce,
					UE::MassAvoidance::CurrentAgentColor, UE::MassAvoidance::SteeringArrowHeadSize, UE::MassAvoidance::SteeringThickness);

				// Draw center
				DebugDrawSphere(BaseDebugContext, AgentLocation, /*Radius*/2.f, UE::MassAvoidance::CurrentAgentColor);
			}
#endif // WITH_MASSGAMEPLAY_DEBUG

			FVector OldSteeringForce = FVector::ZeroVector;

			//////////////////////////////////////////////////////////////////////////
			// Environment avoidance.
			//
			
			if (!MoveTarget.bOffBoundaries && UE::MassAvoidance::Tweakables::bEnableEnvironmentAvoidance)
			{
				const FVector::FReal EnvironmentSeparationAgentRadius = (RadiusFragment.Radius * MovingAvoidanceParams.SeparationRadiusScale) - (NavEdges.bExtrudedEdges ? AgentRadius : 0);
				const FVector::FReal EnvironmentPredictiveAvoidanceAgentRadius = (RadiusFragment.Radius * MovingAvoidanceParams.PredictiveAvoidanceRadiusScale) - (NavEdges.bExtrudedEdges ? AgentRadius : 0);
				
				const FVector DesiredAcceleration = UE::MassNavigation::ClampVector(SteeringForce, MaxSteerAccel);
				const FVector DesiredVelocity = UE::MassNavigation::ClampVector(AgentVelocity + DesiredAcceleration * DeltaTime, MaximumSpeed);

#if WITH_MASSGAMEPLAY_DEBUG
				// Draw desired velocity (yellow)
				UE::MassAvoidance::DebugDrawVelocity(VelocitiesDebugContext, AgentLocation + UE::MassAvoidance::DebugInputForceHeight,
					AgentLocation + UE::MassAvoidance::DebugInputForceHeight + DesiredVelocity, UE::MassAvoidance::DesiredVelocityColor);
#endif // WITH_MASSGAMEPLAY_DEBUG

				OldSteeringForce = SteeringForce;
				Contacts.Reset();

				const FVector::FReal AgentRadiusForEnvironment = NavEdges.bExtrudedEdges ? 0 : AgentRadius;

				int32 EdgeIndex = 0;
				
				// Collect potential contacts between agent and environment edges (obstacles).
				for (const FNavigationAvoidanceEdge& Edge : NavEdges.AvoidanceEdges)
				{
					const FVector EdgeDiff = Edge.End - Edge.Start;
					FVector EdgeDir = FVector::ZeroVector;
					FVector::FReal EdgeLength = 0.;
					EdgeDiff.ToDirectionAndLength(EdgeDir, EdgeLength);

					const FVector EdgeStartToAgent = AgentLocation - Edge.Start;
					const FVector::FReal DistAlongEdge = FVector::DotProduct(EdgeDir, EdgeStartToAgent);
					const FVector::FReal DistAwayFromEdge = FVector::DotProduct(Edge.LeftDir, EdgeStartToAgent);

					FVector ConPos = FVector::ZeroVector; 		// contact position on edge
					bool bBehindTheEdge = false;
					bool bAlongTheEdge = false;

					if (DistAwayFromEdge < 0)
					{
						bBehindTheEdge = true;

						if (DistAlongEdge < 0)
						{
							// Start corner
							ConPos = Edge.Start;
						}
						else if (DistAlongEdge > EdgeLength)
						{
							// End corner
							ConPos = Edge.End;
						}
						else
						{
							// Directly behind the edge
							bAlongTheEdge = true;
							ConPos = Edge.Start + EdgeDir * DistAlongEdge;
						}
					}
					else
					{
						// In front of the edge
						if (DistAlongEdge < 0)
						{
							// Start corner
							ConPos = Edge.Start;
						}
						else if (DistAlongEdge > EdgeLength)
						{
							// End corner
							ConPos = Edge.End;
						}
						else
						{
							// Directly in front of the edge
							bAlongTheEdge = true;
							ConPos = Edge.Start + EdgeDir * DistAlongEdge;
						}
					}
					
					// Add new contact
					FEnvironmentContact Contact;
					Contact.Position = ConPos;
					Contact.Normal = Edge.LeftDir;
					Contact.Distance = DistAwayFromEdge;
					Contact.bAlongTheEdge = bAlongTheEdge;
					Contact.bBehindTheEdge = bBehindTheEdge;
					Contacts.Add(Contact);

#if WITH_MASSGAMEPLAY_DEBUG
					if (UE::MassAvoidance::Tweakables::bEnableDetailedDebug && ObstacleDebugContext.ShouldLogEntity())
					{
						if (bAlongTheEdge)
						{
							// Draw active edges
							FVector ZOffset = FVector(0., 0., 7.);
							DebugDrawLine(ObstacleDebugContext, ZOffset + Edge.Start, ZOffset + Edge.End, FColor::Yellow, /*Thickness=*/3.f,
								/*bPersistent*/false, *LexToString(EdgeIndex));

							// Draw environment separation distance
							const FVector SeparationOffset = MovingAvoidanceParams.EnvironmentSeparationDistance * Edge.LeftDir;
							DebugDrawLine(ObstacleDebugContext, ZOffset + Edge.Start + SeparationOffset, ZOffset + Edge.End + SeparationOffset, FColor::Yellow, /*Thickness=*/1.f);
						}
					
						if (bBehindTheEdge)
						{
							// Draw inactive eges
							FVector Offset = FVector(0., 0., 9.);
							DebugDrawLine(ObstacleDebugContext, Offset + Edge.Start, Offset + Edge.End, FColor::Silver, /*Thickness=*/2.f);	
						}
					}
#endif // WITH_MASSGAMEPLAY_DEBUG
					
					// Skip predictive avoidance when behind the edge.
					if (!bBehindTheEdge)
					{
						// Avoid edges
						
#if WITH_MASSGAMEPLAY_DEBUG
						if (UE::MassAvoidance::Tweakables::bEnableDetailedDebug && ObstacleDebugContext.ShouldLogEntity())
						{
							// Draw environment predictive avoidance distance
							FVector ZOffset = FVector(0., 0., 7.);
							const FVector AvoidanceOffset = MovingAvoidanceParams.EnvironmentPredictiveAvoidanceDistance * Edge.LeftDir;
							DebugDrawLine(ObstacleDebugContext, ZOffset + Edge.Start + AvoidanceOffset, ZOffset + Edge.End + AvoidanceOffset,
								UE::MassAvoidance::ObstacleAvoidForceColor, /*Thickness=*/1.f);
						}
#endif // WITH_MASSGAMEPLAY_DEBUG

						const FVector::FReal CPA = UE::MassAvoidance::ComputeClosestPointOfApproach(
							FVector2D(AgentLocation), FVector2D(DesiredVelocity), AgentRadiusForEnvironment,
							FVector2D(Edge.Start), FVector2D(Edge.End), MovingAvoidanceParams.EnvironmentPredictiveAvoidanceTime);
						const FVector HitAgentPos = AgentLocation + DesiredVelocity * CPA;
						const FVector::FReal EdgeT = UE::MassNavigation::ProjectPtSeg(FVector2D(HitAgentPos), FVector2D(Edge.Start), FVector2D(Edge.End));
						const FVector HitObPos = FMath::Lerp(Edge.Start, Edge.End, EdgeT);

						// Calculate penetration at CPA
						FVector AvoidRelPos = HitAgentPos - HitObPos;
						AvoidRelPos.Z = 0.;	// @todo AT: ignore the z component for now until we clamp the height of obstacles
						const FVector::FReal AvoidDist = AvoidRelPos.Size();
						const FVector AvoidNormal = AvoidDist > UE_KINDA_SMALL_NUMBER ? (AvoidRelPos / AvoidDist) : Edge.LeftDir;

						const FVector::FReal AvoidPen = (EnvironmentPredictiveAvoidanceAgentRadius + MovingAvoidanceParams.EnvironmentPredictiveAvoidanceDistance) - AvoidDist;
						FVector::FReal AvoidMag = 1.;

						if (MovingAvoidanceParams.EnvironmentPredictiveAvoidanceDistance != 0.)
						{
							AvoidMag = FMath::Square(FMath::Clamp(AvoidPen / MovingAvoidanceParams.EnvironmentPredictiveAvoidanceDistance, 0., 1.));
						}

						const FVector::FReal AvoidMagDist = 1. + FMath::Square(1. - CPA * InvEnvironmentPredictiveAvoidanceTime);

						// Predictive avoidance against environment is tuned down towards the end of the path
						const FVector AvoidForce = AvoidNormal * AvoidMag * AvoidMagDist * MovingAvoidanceParams.EnvironmentPredictiveAvoidanceStiffness * NearEndScaling; 

						SteeringForce += AvoidForce;

#if WITH_MASSGAMEPLAY_DEBUG
						if (!AvoidForce.IsNearlyZero())
						{
							if (UE::MassAvoidance::Tweakables::bEnableDetailedDebug)
							{
								// Draw contact normal
								DebugDrawArrow(ObstacleDebugContext, ConPos, ConPos + (Contact.Distance * Contact.Normal), UE::MassAvoidance::ObstacleContactNormalColor, /*HeadSize=*/ 5.f);
								DebugDrawSphere(ObstacleDebugContext, ConPos, 2.5f, UE::MassAvoidance::ObstacleContactNormalColor);
							}

							// Draw future hit pos with edge
							DebugDrawSphere(ObstacleDebugContext, HitAgentPos, 1.f, UE::MassAvoidance::ObstacleAvoidForceColor);
							DebugDrawCircle(ObstacleDebugContext, HitAgentPos, AgentRadius, UE::MassAvoidance::ObstacleAvoidForceColor);
							DebugDrawLine(ObstacleDebugContext, AgentLocation, HitAgentPos, UE::MassAvoidance::ObstacleAvoidForceColor);

							// Draw individual predictive obstacle avoidance forces
							UE::MassAvoidance::DebugDrawForce(ObstacleDebugContext, HitObPos, HitObPos + AvoidForce,
								UE::MassAvoidance::ObstacleAvoidForceColor, UE::MassAvoidance::AvoidThickness);
						}
#endif // WITH_MASSGAMEPLAY_DEBUG
					}

					EdgeIndex++;
					
				} // edge loop

#if WITH_MASSGAMEPLAY_DEBUG
				// Draw total steering force to avoid obstacles
				const FVector EnvironmentAvoidSteeringForce = SteeringForce - OldSteeringForce;
				UE::MassAvoidance::DebugDrawSummedForce(ObstacleDebugContext,
					AgentLocation + UE::MassAvoidance::DebugAgentAvoidHeightOffset,
					AgentLocation + UE::MassAvoidance::DebugAgentAvoidHeightOffset + EnvironmentAvoidSteeringForce,
					UE::MassAvoidance::ObstacleAvoidForceColor);

				if (UE::MassAvoidance::Tweakables::bEnableDetailedDebug)
				{
					// Draw all contact points
					for (const FEnvironmentContact& Contact : Contacts) 
					{
						DebugDrawSphere(ObstacleDebugContext, Contact.Position + FVector(0.f, 0.f, Contact.bBehindTheEdge ? 10.f : 0.f),
							5.f, Contact.bBehindTheEdge ? FColor::Black: FColor::Cyan);					
					}
				}
#endif // WITH_MASSGAMEPLAY_DEBUG
				
				// Process contacts to add edge separation force
				const FVector SteeringForceBeforeSeparation = SteeringForce;

				const FVector::FReal ValidBehindDistance = MovingAvoidanceParams.EnvironmentSeparationBehindEdgeDistanceScale * AgentRadius;
				
				for (const FEnvironmentContact& Contact : Contacts) 
				{
					if (Contact.bAlongTheEdge && Contact.Distance > -ValidBehindDistance)
					{
						// Separation force (stay away from obstacles if possible)
						const FVector::FReal SeparationPenalty = (EnvironmentSeparationAgentRadius + MovingAvoidanceParams.EnvironmentSeparationDistance) - Contact.Distance;

						FVector::FReal SeparationMag;

						const FVector::FReal BehindDistanceThreshold = 0.25*AgentRadius;
						if (Contact.Distance < -BehindDistanceThreshold)
						{
							// Far behind the edge, use a big magnitude to trump other forces.
							SeparationMag = 10.;
						}
						else if (MovingAvoidanceParams.EnvironmentSeparationDistance != 0.)
						{
							SeparationMag = UE::MassNavigation::Smooth(FMath::Clamp(SeparationPenalty / MovingAvoidanceParams.EnvironmentSeparationDistance, 0., 1.));
						}
						else
						{
							SeparationMag = 1.;
						}
						const FVector SeparationForce = Contact.Normal * MovingAvoidanceParams.EnvironmentSeparationStiffness * SeparationMag;
						
						SteeringForce += SeparationForce;

	#if WITH_MASSGAMEPLAY_DEBUG
						UE_VLOG(ObstacleDebugContext.GetLogOwner(), ObstacleDebugContext.Category, Log, TEXT("EnvironmentSeparationAgentRadius: %f, EnvironmentSeparationDistanc: %f, Distance: %f, SeparationPenalty: %f, SeparationMag: %f, SeparationForce: %s"),
							EnvironmentSeparationAgentRadius,
							MovingAvoidanceParams.EnvironmentSeparationDistance,
							Contact.Distance,
							SeparationPenalty,
							SeparationMag,
							*SeparationForce.ToCompactString()
							);
						
						// Draw contact normal
						if (!SeparationForce.IsNearlyZero())
						{
							// Draw individual separation forces
							const FVector ZOffset = FVector(0., 0., 7.);
							UE::MassAvoidance::DebugDrawForce(ObstacleDebugContext, Contact.Position + ZOffset,
								Contact.Position + SeparationForce + ZOffset,
								UE::MassAvoidance::ObstacleSeparationForceColor, UE::MassAvoidance::SeparationThickness);
						}
	#endif // WITH_MASSGAMEPLAY_DEBUG
					}
				}
				
#if WITH_MASSGAMEPLAY_DEBUG
				// Draw total steering force to separate from close edges
				const FVector TotalSeparationForce = SteeringForce - SteeringForceBeforeSeparation;
				UE::MassAvoidance::DebugDrawSummedForce(ObstacleDebugContext,
					AgentLocation + UE::MassAvoidance::DebugAgentSeparationHeightOffset,
					AgentLocation + UE::MassAvoidance::DebugAgentSeparationHeightOffset + TotalSeparationForce,
					UE::MassAvoidance::ObstacleSeparationForceColor);

				// Display close obstacle edges
				if (ObstacleDebugContext.ShouldLogEntity())
				{
					for (const FNavigationAvoidanceEdge& Edge : NavEdges.AvoidanceEdges)
					{
						FVector Offset = FVector(0., 0., 5.);
						DebugDrawLine(ObstacleDebugContext, Offset + Edge.Start, Offset + Edge.End, UE::MassAvoidance::ObstacleColor, /*Thickness=*/2.f);

						const FVector Middle = Offset + 0.5f * (Edge.Start + Edge.End);
						DebugDrawArrow(ObstacleDebugContext, Middle, Middle + 10. * Edge.LeftDir, UE::MassAvoidance::ObstacleColor, /*HeadSize=*/2.f);
					}
				}
#endif // WITH_MASSGAMEPLAY_DEBUG
			}

			//////////////////////////////////////////////////////////////////////////
			// Avoid close agents
			const FVector::FReal SeparationAgentRadius = RadiusFragment.Radius * MovingAvoidanceParams.SeparationRadiusScale;
			const FVector::FReal PredictiveAvoidanceAgentRadius = RadiusFragment.Radius * MovingAvoidanceParams.PredictiveAvoidanceRadiusScale;

			// Update desired velocity based on avoidance so far.
			const FVector DesAcc = UE::MassNavigation::ClampVector(SteeringForce, MaxSteerAccel);
			const FVector DesVel = UE::MassNavigation::ClampVector(AgentVelocity + DesAcc * DeltaTime, MaximumSpeed);

			// Find close obstacles
			const FNavigationObstacleHashGrid2D& AvoidanceObstacleGrid = NavigationSubsystem->GetObstacleGrid();

			UE::MassAvoidance::FindCloseObstacles(AgentLocation, MovingAvoidanceParams.ObstacleDetectionDistance,
				AvoidanceObstacleGrid, CloseEntities, UE::MassAvoidance::MaxObstacleResults);

			// Remove unwanted and find the closests in the CloseEntities
			const FVector::FReal DistanceCutOffSqr = FMath::Square(MovingAvoidanceParams.ObstacleDetectionDistance);
			ClosestObstacles.Reset();
			for (const FNavigationObstacleHashGrid2D::ItemIDType OtherEntity : CloseEntities)
			{
				// Skip self
				if (OtherEntity.Entity == Entity)
				{
					continue;
				}

				// Skip invalid entities.
				if (!EntityManager.IsEntityValid(OtherEntity.Entity))
				{
					UE_LOGF(LogAvoidanceObstacles, VeryVerbose, "Close entity is invalid, skipped.");
					continue;
				}
				
				// Skip too far
				const FTransform& Transform = EntityManager.GetFragmentDataChecked<FTransformFragment>(OtherEntity.Entity).GetTransform();
				const FVector OtherLocation = Transform.GetLocation();
				
				const FVector::FReal SqDist = FVector::DistSquared(AgentLocation, OtherLocation);
				if (SqDist > DistanceCutOffSqr)
				{
					continue;
				}

				// Skip entities to ignore
				if (UNLIKELY(!EntitiesToIgnore.IsEmpty()) && EntitiesToIgnore.Contains(OtherEntity.Entity))
				{
					continue;
				}

				FSortedObstacle Obstacle;
				Obstacle.LocationCached = OtherLocation;
				Obstacle.Forward = Transform.GetRotation().GetForwardVector();
				Obstacle.ObstacleItem = OtherEntity;
				Obstacle.SqDist = SqDist;
				ClosestObstacles.Add(Obstacle);
			}
			ClosestObstacles.Sort([](const FSortedObstacle& A, const FSortedObstacle& B) { return A.SqDist < B.SqDist; });

			// Compute forces
			OldSteeringForce = SteeringForce;
			FVector TotalAgentSeparationForce = FVector::ZeroVector;

			// Fill collider list from close agents
			Colliders.Reset();
			for (int32 Index = 0; Index < ClosestObstacles.Num(); Index++)
			{
				constexpr int32 MaxColliders = 6;
				if (Colliders.Num() >= MaxColliders)
				{
					break;
				}

				FSortedObstacle& Obstacle = ClosestObstacles[Index];
				FMassEntityView OtherEntityView(EntityManager, Obstacle.ObstacleItem.Entity);

				const FMassVelocityFragment* OtherVelocityFragment = OtherEntityView.GetFragmentDataPtr<FMassVelocityFragment>();
				const FVector OtherVelocity = OtherVelocityFragment != nullptr ? OtherVelocityFragment->Value : FVector::ZeroVector; // Get velocity from FAvoidanceComponent

				// @todo: this is heavy fragment to access, see if we could handle this differently.
				const FMassMoveTargetFragment* OtherMoveTarget = OtherEntityView.GetFragmentDataPtr<FMassMoveTargetFragment>();
				const bool bCanAvoid = OtherMoveTarget != nullptr;
				const bool bOtherIsMoving = OtherMoveTarget ? OtherMoveTarget->GetCurrentAction() == EMassMovementAction::Move : true; // Assume moving if other does not have move target.
				
				// Check for colliders data
				if (EnumHasAnyFlags(Obstacle.ObstacleItem.ItemFlags, EMassNavigationObstacleFlags::HasColliderData))
				{
					if (const FMassAvoidanceColliderFragment* ColliderFragment = OtherEntityView.GetFragmentDataPtr<FMassAvoidanceColliderFragment>())
					{
						if (ColliderFragment->Type == EMassColliderType::Circle)
						{
							const FMassCircleCollider Circle = ColliderFragment->GetCircleCollider();
							
							FCollider& Collider = Colliders.Add_GetRef(FCollider{});
							Collider.Velocity = OtherVelocity;
							Collider.bCanAvoid = bCanAvoid;
							Collider.bIsMoving = bOtherIsMoving;
							Collider.Radius = Circle.Radius;
							Collider.Location = Obstacle.LocationCached;
						}
						else if (ColliderFragment->Type == EMassColliderType::Pill)
						{
							const FMassPillCollider Pill = ColliderFragment->GetPillCollider(); 

							FCollider& Collider = Colliders.Add_GetRef(FCollider{});
							Collider.Velocity = OtherVelocity;
							Collider.bCanAvoid = bCanAvoid;
							Collider.bIsMoving = bOtherIsMoving;
							Collider.Radius = Pill.Radius;
							Collider.Location = Obstacle.LocationCached + (Pill.HalfLength * Obstacle.Forward);

							if (Colliders.Num() < MaxColliders)
							{
								FCollider& Collider2 = Colliders.Add_GetRef(FCollider{});
								Collider2.Velocity = OtherVelocity;
								Collider2.bCanAvoid = bCanAvoid;
								Collider2.bIsMoving = bOtherIsMoving;
								Collider2.Radius = Pill.Radius;
								Collider2.Location = Obstacle.LocationCached + (-Pill.HalfLength * Obstacle.Forward);
							}
						}
					}
				}
				else
				{
					FCollider& Collider = Colliders.Add_GetRef(FCollider{});
					Collider.Location = Obstacle.LocationCached;
					Collider.Velocity = OtherVelocity;
					Collider.Radius = OtherEntityView.GetFragmentData<FAgentRadiusFragment>().Radius;
					Collider.bCanAvoid = bCanAvoid;
					Collider.bIsMoving = bOtherIsMoving;
				}
			}

			// Process colliders for avoidance
			for (const FCollider& Collider : Colliders)
			{
				bool bHasForcedNormal = false;
				FVector ForcedNormal = FVector::ZeroVector;

				if (Collider.bCanAvoid == false)
				{
					// If the other obstacle cannot avoid us, try to avoid the local minima they create between the wall and their collider.
					// If the space between edge and collider is less than MinClearance, make the agent to avoid the gap.
					const FVector::FReal MinClearance = 2. * AgentRadius * MovingAvoidanceParams.StaticObstacleClearanceScale;
					
					// Find the maximum distance from edges that are too close.
					FVector::FReal MaxDist = -1.;
					FVector ClosestPoint = FVector::ZeroVector;
					for (const FNavigationAvoidanceEdge& Edge : NavEdges.AvoidanceEdges)
					{
						const FVector Point = FMath::ClosestPointOnSegment(Collider.Location, Edge.Start, Edge.End);
						const FVector Offset = Collider.Location - Point;
						if (FVector::DotProduct(Offset, Edge.LeftDir) < 0.)
						{
							// Behind the edge, ignore.
							continue;
						}

						const FVector::FReal OffsetLength = Offset.Length();
						const bool bTooNarrow = (OffsetLength - Collider.Radius) < MinClearance; 
						if (bTooNarrow)
						{
							if (OffsetLength > MaxDist)
							{
								MaxDist = OffsetLength;
								ClosestPoint = Point;
							}
						}
					}

					if (MaxDist != -1.)
					{
						// Set up forced normal to avoid the gap between collider and edge.
						ForcedNormal = (Collider.Location - ClosestPoint).GetSafeNormal();
						bHasForcedNormal = true;
					}
				}

				FVector RelPos = AgentLocation - Collider.Location;
				RelPos.Z = 0.; // we assume we work on a flat plane for now
				const FVector RelVel = DesVel - Collider.Velocity;
				const FVector::FReal ConDist = RelPos.Size();
				const FVector ConNorm = ConDist > 0. ? RelPos / ConDist : FVector::ForwardVector;

				FVector SeparationNormal = ConNorm;
				if (bHasForcedNormal)
				{
					// The more head on the collisions is, the more we should avoid towards the forced direction.
					const FVector RelVelNorm = RelVel.GetSafeNormal();
					const FVector::FReal Blend = FMath::Max(0., -FVector::DotProduct(ConNorm, RelVelNorm));
					SeparationNormal = FMath::Lerp(ConNorm, ForcedNormal, Blend).GetSafeNormal();
				}

				// Care less about standing agents so that we can push through standing crowd.
				const FVector::FReal StandingScaling = Collider.bIsMoving ? 1. : MovingAvoidanceParams.StandingObstacleAvoidanceScale; 
				
				// Separation force (stay away from agents if possible)
				const FVector::FReal PenSep = (SeparationAgentRadius + Collider.Radius + MovingAvoidanceParams.ObstacleSeparationDistance) - ConDist;
				const FVector::FReal SeparationMag = FMath::Square(FMath::Clamp(PenSep / MovingAvoidanceParams.ObstacleSeparationDistance, 0., 1.));
				const FVector SepForce = SeparationNormal * MovingAvoidanceParams.ObstacleSeparationStiffness;
				const FVector SeparationForce = SepForce * SeparationMag * StandingScaling;

				SteeringForce += SeparationForce;
				TotalAgentSeparationForce += SeparationForce;

				// Calculate the closest point of approach based on relative agent positions and velocities.
				const FVector::FReal CPA = UE::MassAvoidance::ComputeClosestPointOfApproach(RelPos, RelVel, PredictiveAvoidanceAgentRadius + Collider.Radius, MovingAvoidanceParams.PredictiveAvoidanceTime);

				// Calculate penetration at CPA
				const FVector AvoidRelPos = RelPos + RelVel * CPA;
				const FVector::FReal AvoidDist = AvoidRelPos.Size();
				const FVector AvoidConNormal = AvoidDist > UE_KINDA_SMALL_NUMBER ? (AvoidRelPos / AvoidDist) : FVector::ForwardVector;

				FVector AvoidNormal = AvoidConNormal;
				if (bHasForcedNormal)
				{
					// The more head on the predicted collisions is, the more we should avoid towards the forced direction.
					const FVector RelVelNorm = RelVel.GetSafeNormal();
					const FVector::FReal Blend = FMath::Max(0., -FVector::DotProduct(AvoidConNormal, RelVelNorm));
					AvoidNormal = FMath::Lerp(AvoidConNormal, ForcedNormal, Blend).GetSafeNormal();
				}
				
				const FVector::FReal AvoidPenetration = (PredictiveAvoidanceAgentRadius + Collider.Radius + MovingAvoidanceParams.PredictiveAvoidanceDistance) - AvoidDist; // Based on future agents distance
				const FVector::FReal AvoidMag = FMath::Square(FMath::Clamp(AvoidPenetration / MovingAvoidanceParams.PredictiveAvoidanceDistance, 0., 1.));
				const FVector::FReal AvoidMagDist = (1. - (CPA * InvPredictiveAvoidanceTime)); // No clamp, CPA is between 0 and PredictiveAvoidanceTime
				const FVector AvoidForce = AvoidNormal * AvoidMag * AvoidMagDist * MovingAvoidanceParams.ObstaclePredictiveAvoidanceStiffness * StandingScaling;

				SteeringForce += AvoidForce;

#if WITH_MASSGAMEPLAY_DEBUG
				// Display close agent
				UE::MassAvoidance::DebugDrawCylinder(AgentDebugContext, Collider.Location,
					Collider.Location + UE::MassAvoidance::DebugLowCylinderOffset, Collider.Radius, UE::MassAvoidance::AgentsColor);

				if (bHasForcedNormal)
				{
					UE::MassAvoidance::DebugDrawCylinder(BaseDebugContext, Collider.Location,
						Collider.Location + UE::MassAvoidance::DebugAgentHeightOffset, Collider.Radius, FColor::Red);
				}

				// Draw agent contact separation force
				if (!SeparationForce.IsNearlyZero())
				{
					UE::MassAvoidance::DebugDrawForce(AgentDebugContext,
						Collider.Location + UE::MassAvoidance::DebugAgentSeparationHeightOffset,
						Collider.Location + UE::MassAvoidance::DebugAgentSeparationHeightOffset + SeparationForce,
						Collider.bIsMoving ? UE::MassAvoidance::AgentSeparationForceColor : UE::MassAvoidance::AgentsNotMovingSeparationColor,
						UE::MassAvoidance::SeparationThickness);
				}

				if (UE::MassAvoidance::Tweakables::bEnableDetailedDebug)
				{
					UE::MassAvoidance::DebugDrawCircle(AgentDebugContext, Collider.Location, Collider.Radius + MovingAvoidanceParams.PredictiveAvoidanceDistance, UE::MassAvoidance::AgentsColor);
					UE::MassAvoidance::DebugDrawCircle(AgentDebugContext, Collider.Location, Collider.Radius + MovingAvoidanceParams.ObstacleSeparationDistance, UE::MassAvoidance::AgentSeparationForceColor);
				}
				
				if (AvoidForce.Size() > 0.)
				{
					// Draw agent vs agent hit positions
					UE::MassAvoidance::DebugDrawHitPosition(AgentDebugContext, AgentLocation, DesVel, CPA, PredictiveAvoidanceAgentRadius, UE::MassAvoidance::CurrentAgentColor);

					// Draw collider hit position
					const FColor ColliderColor = Collider.bIsMoving ? UE::MassAvoidance::AgentsColor : UE::MassAvoidance::AgentsNotMovingColor;
					UE::MassAvoidance::DebugDrawHitPosition(AgentDebugContext, Collider.Location, Collider.Velocity, CPA, Collider.Radius, ColliderColor);

					const FVector OtherHitPosition = Collider.Location + (Collider.Velocity * CPA);

					UE::MassAvoidance::DebugDrawCylinder(AgentDebugContext, Collider.Location, Collider.Location + UE::MassAvoidance::DebugAgentHeightOffset,
						AgentRadius, ColliderColor);

					// Draw agent avoid force
					UE::MassAvoidance::DebugDrawForce(AgentDebugContext,
						OtherHitPosition + UE::MassAvoidance::DebugAgentAvoidHeightOffset,
						OtherHitPosition + UE::MassAvoidance::DebugAgentAvoidHeightOffset + AvoidForce,
						UE::MassAvoidance::AgentAvoidForceColor, UE::MassAvoidance::AvoidThickness);
				}
#endif // WITH_MASSGAMEPLAY_DEBUG
			} // close entities loop

			// Only scale the added avoidance force (ignoring the original steering force) and apply it back into the steering force.
			FVector AccumulatedAvoidanceForce = SteeringForce - Force.Value;
			AccumulatedAvoidanceForce *= NearStartScaling * NearEndScaling;
			SteeringForce = Force.Value + AccumulatedAvoidanceForce;

			// Make sure that the avoidance force will not bring the desired velocity backward.
			if (bPreventBackwardMovement && bHasDesiredMovementFragments)
			{
				FMassDesiredMovementFragment& DesiredMovement = DesiredMovementList[EntityIt];
				
				const FVector TempDesiredVelocity  = DesiredMovement.DesiredVelocity + (UE::MassNavigation::ClampVector(SteeringForce, MaxSteerAccel) * DeltaTime);
				const FVector PredictedDesiredVelocity = TempDesiredVelocity.GetClampedToMaxSize(MovementParams.MaxSpeed);

				constexpr FVector::FReal Cos45 = 0.707;
				const FVector::FReal Dot = PredictedDesiredVelocity.GetSafeNormal().Dot(MoveTarget.Forward);
				if (Dot < Cos45)
				{
					// That force would bring the DesiredVelocity in an unwanted range. 
					// Ignore it, but keep the component in the direction of movement.
					const FVector::FReal ForwardComponent = FMath::Abs(SteeringForce.Dot(MoveTarget.Forward));
					SteeringForce = ForwardComponent * MoveTarget.Forward;
				}

				// Snap small speeds to 0.
				constexpr FVector::FReal MinSpeed = 20;
				const bool bSlowing = PredictedDesiredVelocity.SizeSquared() < DesiredMovement.DesiredVelocity.SizeSquared();
				if (bSlowing && PredictedDesiredVelocity.Size() < MinSpeed)
				{
					DesiredMovement.DesiredVelocity = FVector::ZeroVector;
					SteeringForce = FVector::ZeroVector;
				}
			}

			Force.Value = UE::MassNavigation::ClampVector(SteeringForce, MaxSteerAccel); // Assume unit mass

#if WITH_MASSGAMEPLAY_DEBUG
			const FVector AgentAvoidSteeringForce = SteeringForce - OldSteeringForce;

			// Draw total steering force to separate agents
			UE::MassAvoidance::DebugDrawSummedForce(AgentDebugContext,
				AgentLocation + UE::MassAvoidance::DebugAgentSeparationHeightOffset,
				AgentLocation + UE::MassAvoidance::DebugAgentSeparationHeightOffset + TotalAgentSeparationForce,
				UE::MassAvoidance::AgentSeparationForceColor);

			// Draw total steering force to avoid agents
			UE::MassAvoidance::DebugDrawSummedForce(AgentDebugContext,
				AgentLocation + UE::MassAvoidance::DebugAgentAvoidHeightOffset,
				AgentLocation + UE::MassAvoidance::DebugAgentAvoidHeightOffset + AgentAvoidSteeringForce,
				UE::MassAvoidance::AgentAvoidForceColor);

			// Draw final steering force adding
			UE::MassAvoidance::DebugDrawArrow(BaseDebugContext, 
				AgentLocation + UE::MassAvoidance::DebugOutputForcesHeight,
				AgentLocation + UE::MassAvoidance::DebugOutputForcesHeight + Force.Value,
				UE::MassAvoidance::FinalSteeringForceColor, UE::MassAvoidance::SteeringArrowHeadSize, UE::MassAvoidance::SteeringThickness);
#endif // WITH_MASSGAMEPLAY_DEBUG
		}
	});
}

//----------------------------------------------------------------------//
//  UMassStandingAvoidanceProcessor
//----------------------------------------------------------------------//
UMassStandingAvoidanceProcessor::UMassStandingAvoidanceProcessor()
	: EntityQuery(*this)
{
	bAutoRegisterWithProcessingPhases = true;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Avoidance;
	ExecutionOrder.ExecuteAfter.Add(TEXT("MassMovingAvoidanceProcessor"));
}

void UMassStandingAvoidanceProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassGhostLocationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassNavigationEdgesFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassAvoidanceEntitiesToIgnoreFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery.AddTagRequirement<FMassMediumLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassLowLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddConstSharedRequirement<FMassStandingAvoidanceParameters>(EMassFragmentPresence::All);
	EntityQuery.AddSubsystemRequirement<UMassNavigationSubsystem>(EMassFragmentAccess::ReadOnly);

	// Indirect access: fragments read on neighbor entities returned by FindCloseObstacles
	EntityQuery.AddIndirectFragmentRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddIndirectFragmentRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddIndirectFragmentRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddIndirectFragmentRequirement<FMassGhostLocationFragment>(EMassFragmentAccess::ReadOnly);

#if WITH_MASSGAMEPLAY_DEBUG
	EntityQuery.DebugEnableEntityOwnerLogging();
#endif
}

void UMassStandingAvoidanceProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(UMassStandingAvoidanceProcessor);

	UWorld* DebugDrawWorld = UE::Mass::Debug::GetDebugDrawWorld(Context);

	// Avoidance while standing
	EntityQuery.ForEachEntityChunk(Context, [FallbackLogOwner = this, &EntityManager, DebugDrawWorld](FMassExecutionContext& Context)
	{
		const UMassNavigationSubsystem* NavigationSubsystem = Context.GetSubsystem<UMassNavigationSubsystem>();
		if (!NavigationSubsystem)
		{
			return;
		}
		const float DeltaTime = Context.GetDeltaTimeSeconds();

		const TArrayView<FMassGhostLocationFragment> GhostList = Context.GetMutableFragmentView<FMassGhostLocationFragment>();
		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		const TConstArrayView<FAgentRadiusFragment> RadiusList = Context.GetFragmentView<FAgentRadiusFragment>();
		const TConstArrayView<FMassAvoidanceEntitiesToIgnoreFragment> EntitiesToIgnoreList = Context.GetFragmentView<FMassAvoidanceEntitiesToIgnoreFragment>();
		const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();
		const FMassStandingAvoidanceParameters& StandingParams = Context.GetConstSharedFragment<FMassStandingAvoidanceParameters>();

		const FVector::FReal GhostSeparationDistance = StandingParams.GhostSeparationDistance;
		const FVector::FReal GhostSeparationStiffness = StandingParams.GhostSeparationStiffness;

		const FVector::FReal MovingSeparationDistance = StandingParams.GhostSeparationDistance * StandingParams.MovingObstacleAvoidanceScale;
		const FVector::FReal MovingSeparationStiffness = StandingParams.GhostSeparationStiffness * StandingParams.MovingObstacleAvoidanceScale;

		// Arrays used to store close agents
		TArray<FMassNavigationObstacleItem, TFixedAllocator<UE::MassAvoidance::MaxObstacleResults>> CloseEntities;

		struct FSortedObstacle
		{
			FSortedObstacle() = default;
			FSortedObstacle(const FMassEntityHandle InEntity, const FVector InLocation, const FVector InForward, const FVector::FReal InDistSq)
				: Entity(InEntity)
				, Location(InLocation)
				, Forward(InForward)
				, DistSq(InDistSq)
			{}
			
			FMassEntityHandle Entity;
			FVector Location = FVector::ZeroVector;
			FVector Forward = FVector::ForwardVector;
			FVector::FReal DistSq = 0.;
		};
		TArray<FSortedObstacle, TFixedAllocator<UE::MassAvoidance::MaxObstacleResults>> ClosestObstacles;

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// @todo: this check should eventually be part of the query.
			const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
			if (MoveTarget.GetCurrentAction() != EMassMovementAction::Stand)
			{
				continue;
			}
			
			FMassGhostLocationFragment& Ghost = GhostList[EntityIt];
			// Skip if the ghost is not valid for this movement action yet.
			if (Ghost.IsValid(MoveTarget.GetCurrentActionID()) == false)
			{
				continue;
			}

			const FTransformFragment& Location = LocationList[EntityIt];
			const FAgentRadiusFragment& Radius = RadiusList[EntityIt];
			
			TConstArrayView<FMassEntityHandle> EntitiesToIgnore =
				EntitiesToIgnoreList.IsEmpty() ?
				TConstArrayView<FMassEntityHandle>() :
					EntitiesToIgnoreList[EntityIt].EntitiesToIgnore;

			FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			const FVector AgentLocation = Location.GetTransform().GetTranslation();
			const FVector::FReal AgentRadius = Radius.Radius;

			// Steer ghost to move target.
			const FVector::FReal SteerK = 1. / StandingParams.GhostSteeringReactionTime;
			constexpr FVector::FReal SteeringMinDistance = 1.; // Do not bother to steer if the distance is less than this.

			FVector SteerDirection = FVector::ZeroVector;
			FVector Delta = MoveTarget.Center - Ghost.Location;
			Delta.Z = 0.;
			const FVector::FReal Distance = Delta.Size();
			FVector::FReal SpeedFade = 0.;
			if (Distance > SteeringMinDistance)
			{
				SteerDirection = Delta / Distance;
				SpeedFade = FMath::Clamp(Distance / FMath::Max(KINDA_SMALL_NUMBER, StandingParams.GhostStandSlowdownRadius), 0., 1.);
			}

			const FVector GhostDesiredVelocity = SteerDirection * StandingParams.GhostMaxSpeed * SpeedFade;
			FVector GhostSteeringForce = SteerK * (GhostDesiredVelocity - Ghost.Velocity); // Goal force
			
			// Find close obstacles
			// @todo: optimize FindCloseObstacles() and cache results. We're intentionally using agent location here, to allow to share the results with moving avoidance.
			const FNavigationObstacleHashGrid2D& ObstacleGrid = NavigationSubsystem->GetObstacleGrid();
			UE::MassAvoidance::FindCloseObstacles(AgentLocation, StandingParams.GhostObstacleDetectionDistance, ObstacleGrid, CloseEntities, UE::MassAvoidance::MaxObstacleResults);

			// Remove unwanted and find the closest in the CloseEntities
			const FVector::FReal DistanceCutOffSqr = FMath::Square(StandingParams.GhostObstacleDetectionDistance);
			ClosestObstacles.Reset();
			for (const FNavigationObstacleHashGrid2D::ItemIDType OtherEntity : CloseEntities)
			{
				// Skip self
				if (OtherEntity.Entity == Entity)
				{
					continue;
				}

				// Skip invalid entities.
				if (!EntityManager.IsEntityValid(OtherEntity.Entity))
				{
					UE_LOGF(LogAvoidanceObstacles, VeryVerbose, "Close entity is invalid, skipped.");
					continue;
				}

				// Skip too far
				const FTransformFragment& OtherTransform = EntityManager.GetFragmentDataChecked<FTransformFragment>(OtherEntity.Entity);
				const FVector OtherLocation = OtherTransform.GetTransform().GetLocation();
				const FVector::FReal DistSq = FVector::DistSquared(AgentLocation, OtherLocation);
				if (DistSq > DistanceCutOffSqr)
				{
					continue;
				}

				// Skip entities to ignore
				if (UNLIKELY(!EntitiesToIgnore.IsEmpty()) && EntitiesToIgnore.Contains(OtherEntity.Entity))
				{
					continue;
				}
				
				ClosestObstacles.Emplace(OtherEntity.Entity, OtherLocation, OtherTransform.GetTransform().GetRotation().GetForwardVector(), DistSq);
			}
			ClosestObstacles.Sort([](const FSortedObstacle& A, const FSortedObstacle& B) { return A.DistSq < B.DistSq; });

			const FVector::FReal GhostRadius = AgentRadius * StandingParams.GhostSeparationRadiusScale;

#if WITH_MASSGAMEPLAY_DEBUG
			const UE::MassAvoidance::FDebugContext AgentDebugContext(Context, FallbackLogOwner, LogAvoidanceAgents, DebugDrawWorld, Entity, EntityIt);

			// Draw entity
			UE::MassAvoidance::DebugDrawCylinder(AgentDebugContext, AgentLocation, AgentLocation + UE::MassAvoidance::DebugAgentHeightOffset,
				AgentRadius, UE::MassAvoidance::CurrentAgentColor);

			// Draw ghost
			UE::MassAvoidance::DebugDrawCylinder(AgentDebugContext, Ghost.Location, Ghost.Location + UE::MassAvoidance::DebugAgentHeightOffset,
				AgentRadius, UE::MassAvoidance::GhostColor);
#endif // WITH_MASSGAMEPLAY_DEBUG
			
			// Compute forces
			constexpr int32 MaxCloseObstacleTreated = 6;
			const int32 NumCloseObstacles = FMath::Min(ClosestObstacles.Num(), MaxCloseObstacleTreated);
			for (int32 Index = 0; Index < NumCloseObstacles; Index++)
			{
				FSortedObstacle& OtherAgent = ClosestObstacles[Index];
				FMassEntityView OtherEntityView(EntityManager, OtherAgent.Entity);

				const FVector::FReal OtherRadius = OtherEntityView.GetFragmentData<FAgentRadiusFragment>().Radius;
				const FVector::FReal TotalRadius = GhostRadius + OtherRadius;

#if WITH_MASSGAMEPLAY_DEBUG
				// Draw other agent
				UE::MassAvoidance::DebugDrawCylinder(AgentDebugContext, OtherAgent.Location, OtherAgent.Location + UE::MassAvoidance::DebugAgentHeightOffset,
					OtherRadius, UE::MassAvoidance::AgentsColor);
#endif // WITH_MASSGAMEPLAY_DEBUG
				
				// @todo: this is heavy fragment to access, see if we could handle this differently.
				const FMassMoveTargetFragment* OtherMoveTarget = OtherEntityView.GetFragmentDataPtr<FMassMoveTargetFragment>();
				const FMassGhostLocationFragment* OtherGhost = OtherEntityView.GetFragmentDataPtr<FMassGhostLocationFragment>();

				const bool bOtherHasGhost = OtherMoveTarget != nullptr && OtherGhost != nullptr
											&& OtherMoveTarget->GetCurrentAction() == EMassMovementAction::Stand
											&& OtherGhost->IsValid(OtherMoveTarget->GetCurrentActionID());

				// If other has ghost active, avoid that, else avoid the actual agent.
				if (bOtherHasGhost)
				{
					// Avoid the other agent more, when it is further away from its goal location.
					const FVector::FReal OtherDistanceToGoal = FVector::Distance(OtherGhost->Location, OtherMoveTarget->Center);
					const FVector::FReal OtherSteerFade = FMath::Clamp(OtherDistanceToGoal / StandingParams.GhostToTargetMaxDeviation, 0., 1.);
					const FVector::FReal SeparationStiffness = FMath::Lerp(GhostSeparationStiffness, MovingSeparationStiffness, OtherSteerFade);

					// Ghost separation
					FVector RelPos = Ghost.Location - OtherGhost->Location;
					RelPos.Z = 0.; // we assume we work on a flat plane for now
					const FVector::FReal ConDist = RelPos.Size();
					const FVector ConNorm = ConDist > 0. ? RelPos / ConDist : FVector::ForwardVector;

					// Separation force (stay away from obstacles if possible)
					const FVector::FReal PenSep = (TotalRadius + GhostSeparationDistance) - ConDist;
					const FVector::FReal SeparationMag = UE::MassNavigation::Smooth(FMath::Clamp(PenSep / GhostSeparationDistance, 0., 1.));
					const FVector SeparationForce = ConNorm * SeparationStiffness * SeparationMag;

					GhostSteeringForce += SeparationForce;

#if WITH_MASSGAMEPLAY_DEBUG					
					// Draw agent avoid force
					UE::MassAvoidance::DebugDrawForce(AgentDebugContext,
						Ghost.Location + UE::MassAvoidance::DebugAgentHeightOffset,
						Ghost.Location + UE::MassAvoidance::DebugAgentHeightOffset + SeparationForce,
						UE::MassAvoidance::AgentAvoidForceColor, UE::MassAvoidance::AvoidThickness);
#endif // WITH_MASSGAMEPLAY_DEBUG
				}
				else
				{
					// Avoid more when the avoidance other is in front,
					const FVector DirToOther = (OtherAgent.Location - Ghost.Location).GetSafeNormal();
					const FVector::FReal DirectionalFade = FMath::Square(FMath::Max(0., FVector::DotProduct(MoveTarget.Forward, DirToOther)));
					const FVector::FReal DirectionScale = FMath::Lerp(StandingParams.MovingObstacleDirectionalScale, 1., DirectionalFade);

					// Treat the other agent as a 2D capsule protruding towards forward.
 					const FVector OtherBasePosition = OtherAgent.Location;
					const FVector OtherPersonalSpacePosition = OtherAgent.Location + OtherAgent.Forward * OtherRadius * StandingParams.MovingObstaclePersonalSpaceScale * DirectionScale;
					const FVector OtherLocation = FMath::ClosestPointOnSegment(Ghost.Location, OtherBasePosition, OtherPersonalSpacePosition);

					FVector RelPos = Ghost.Location - OtherLocation;
					RelPos.Z = 0.;
					const FVector::FReal ConDist = RelPos.Size();
					const FVector ConNorm = ConDist > 0. ? RelPos / ConDist : FVector::ForwardVector;

					// Separation force (stay away from obstacles if possible)
					const FVector::FReal PenSep = (TotalRadius + MovingSeparationDistance) - ConDist;
					const FVector::FReal SeparationMag = UE::MassNavigation::Smooth(FMath::Clamp(PenSep / MovingSeparationDistance, 0., 1.));
					const FVector SeparationForce = ConNorm * MovingSeparationStiffness * SeparationMag;

					GhostSteeringForce += SeparationForce;

#if WITH_MASSGAMEPLAY_DEBUG
					// Draw agent avoid force
					UE::MassAvoidance::DebugDrawForce(AgentDebugContext,
						OtherBasePosition + UE::MassAvoidance::DebugAgentHeightOffset,
						OtherBasePosition + UE::MassAvoidance::DebugAgentHeightOffset + SeparationForce,
						UE::MassAvoidance::AgentAvoidForceColor, UE::MassAvoidance::AvoidThickness);
#endif // WITH_MASSGAMEPLAY_DEBUG
				}
			}

			GhostSteeringForce.Z = 0.;
			GhostSteeringForce = UE::MassNavigation::ClampVector(GhostSteeringForce, StandingParams.GhostMaxAcceleration); // Assume unit mass

#if WITH_MASSGAMEPLAY_DEBUG
			// Draw total steering force
			UE::MassAvoidance::DebugDrawSummedForce(AgentDebugContext,
				AgentLocation + UE::MassAvoidance::DebugAgentHeightOffset,
				AgentLocation + UE::MassAvoidance::DebugAgentHeightOffset + GhostSteeringForce,
				UE::MassAvoidance::GhostSteeringForceColor);
#endif // WITH_MASSGAMEPLAY_DEBUG
			
			Ghost.Velocity += GhostSteeringForce * DeltaTime;
			Ghost.Velocity.Z = 0.;
			
			// Damping
			FMath::ExponentialSmoothingApprox(Ghost.Velocity, FVector::ZeroVector, DeltaTime, StandingParams.GhostVelocityDampingTime);
			
			Ghost.Location += Ghost.Velocity * DeltaTime;

			// Don't let the ghost location too far from move target center.
			const FVector DirToCenter = Ghost.Location - MoveTarget.Center;
			const FVector::FReal DistToCenter = DirToCenter.Length();
			if (DistToCenter > StandingParams.GhostToTargetMaxDeviation)
			{
				Ghost.Location = MoveTarget.Center + DirToCenter * (StandingParams.GhostToTargetMaxDeviation / DistToCenter);
			}
		}
	});
	
}

