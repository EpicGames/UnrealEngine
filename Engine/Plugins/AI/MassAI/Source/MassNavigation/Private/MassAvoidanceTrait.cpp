// Copyright Epic Games, Inc. All Rights Reserved.

#include "Avoidance/MassAvoidanceTrait.h"
#include "Avoidance/MassAvoidanceFragments.h"
#include "MassEntityTemplateRegistry.h"
#include "MassMovementFragments.h"
#include "MassCommonFragments.h"
#include "MassNavigationFragments.h"
#include "Engine/World.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassAvoidanceTrait)


void UMassObstacleAvoidanceTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	BuildContext.RequireFragment<FAgentRadiusFragment>();
	BuildContext.AddFragment<FMassNavigationEdgesFragment>();
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.RequireFragment<FMassVelocityFragment>();
	BuildContext.RequireFragment<FMassForceFragment>();
	BuildContext.RequireFragment<FMassMoveTargetFragment>();

	const FMassMovingAvoidanceParameters MovingValidated = MovingParameters.GetValidated();
	const FConstSharedStruct MovingFragment = EntityManager.GetOrCreateConstSharedFragment(MovingValidated);
	BuildContext.AddConstSharedFragment(MovingFragment);

	const FMassStandingAvoidanceParameters StandingValidated = StandingParameters.GetValidated();
	const FConstSharedStruct StandingFragment = EntityManager.GetOrCreateConstSharedFragment(StandingValidated);
	BuildContext.AddConstSharedFragment(StandingFragment);
}
