// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavigationDebug.h"

#if WITH_MASSGAMEPLAY_DEBUG

#include "MassDebugger.h"
#include "VisualLogger/VisualLogger.h"
#include "DrawDebugHelpers.h"

namespace UE::MassNavigation::Debug
{
	namespace Tweakables
	{
		bool bUseDrawDebugHelpers = false;
		bool bLogEverythingWhenRecording = false;
	} // Tweakables

	FAutoConsoleVariableRef Vars[] = 
	{
		FAutoConsoleVariableRef(TEXT("ai.mass.debug.UseDrawDebugHelpers"), Tweakables::bUseDrawDebugHelpers, TEXT("Use debug draw helpers in addition to visual logs."), ECVF_Cheat),
		FAutoConsoleVariableRef(TEXT("ai.mass.debug.LogEverythingWhenRecording"), Tweakables::bLogEverythingWhenRecording, TEXT("If true, will log all debug draw events regardless of debug entity selection if the visual log recorder is activated"), ECVF_Cheat)
	};
	
	bool UseDrawDebugHelper()
	{
		return Tweakables::bUseDrawDebugHelpers;
	}

	bool ShouldLogEverythingWhenRecording()
	{
		return Tweakables::bLogEverythingWhenRecording;
	}

	void DebugDrawLine(const FDebugContext& Context, const FVector& Start, const FVector& End, const FColor& Color, const float Thickness /*= 0.f*/, const bool bPersistent /*= false*/, const FString& Text /*= FString()*/)
	{
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		UE_VLOG_SEGMENT_THICK(Context.GetLogOwner(), Context.Category, Log, Start, End, Color, (int16)Thickness, TEXT("%s"), *Text);

		if (UseDrawDebugHelper() && Context.World)
		{
			DrawDebugLine(Context.World, Start, End, Color, bPersistent, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
		}
	}

	void DebugDrawArrow(const FDebugContext& Context, const FVector& Start, const FVector& End, const FColor& Color,
		const float HeadSize /*= 8.f*/, const float Thickness /*= 1.5f*/, const FString& Text /*= FString()*/)
	{
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		const UObject* LogOwner = Context.GetLogOwner();

		constexpr FVector::FReal Pointyness = 1.8;
		const FVector Line = End - Start;
		const FVector UnitV = Line.GetSafeNormal();
		const FVector Perp = FVector::CrossProduct(UnitV, FVector::UpVector);
		const FVector Left = Perp - (Pointyness*UnitV);
		const FVector Right = -Perp - (Pointyness*UnitV);
		UE_VLOG_SEGMENT_THICK(LogOwner, Context.Category, Log, Start, End, Color, (int16)Thickness, TEXT("%s"), *Text);
		UE_VLOG_SEGMENT_THICK(LogOwner, Context.Category, Log, End, End + HeadSize * Left, Color, (int16)Thickness, TEXT(""));
		UE_VLOG_SEGMENT_THICK(LogOwner, Context.Category, Log, End, End + HeadSize * Right, Color, (int16)Thickness, TEXT(""));

		if (UseDrawDebugHelper() && Context.World)
		{
			DrawDebugLine(Context.World, Start, End, Color, /*bPersistent=*/ false, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
			DrawDebugLine(Context.World, End, End + HeadSize * Left, Color, /*bPersistent=*/ false, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
			DrawDebugLine(Context.World, End, End + HeadSize * Right, Color, /*bPersistent=*/ false, /*LifeTime =*/ -1.f, /*DepthPriority =*/ 0, Thickness);
		}
	}

	void DebugDrawSphere(const FDebugContext& Context, const FVector& Center, const FVector::FReal InRadius, const FColor& Color, const FString& Text /*= FString()*/)
	{
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		const float Radius = static_cast<float>(InRadius);
		UE_VLOG_LOCATION(Context.GetLogOwner(), Context.Category, Log, Center, Radius, Color, TEXT("%s"), *Text);

		if (UseDrawDebugHelper() && Context.World)
		{
			DrawDebugSphere(Context.World, Center, Radius, /*Segments = */16, Color);
		}
	}

	void DebugDrawBox(const FDebugContext& Context, const FBox& Box, const FColor& Color, const FString& Text /*= FString()*/)
	{
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		UE_VLOG_BOX(Context.GetLogOwner(), Context.Category, Log, Box, Color, TEXT("%s"), *Text);
		
		if (UseDrawDebugHelper() && Context.World)
		{
			DrawDebugBox(Context.World, Box.GetCenter(), Box.GetExtent(), Color);
		}
	}
	
	void DebugDrawCylinder(const FDebugContext& Context, const FVector& Bottom, const FVector& Top, const FVector::FReal InRadius, const FColor& Color, const FString& Text /*= FString()*/)
	{
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		const float Radius = static_cast<float>(InRadius);
		UE_VLOG_CYLINDER(Context.GetLogOwner(), Context.Category, Log, Bottom, Top, Radius, Color, TEXT("%s"), *Text);

		if (UseDrawDebugHelper() && Context.World)
		{
			DrawDebugCylinder(Context.World, Bottom, Top, Radius, /*Segments = */24, Color);
		}
	}
	
	void DebugDrawCircle(const FDebugContext& Context, const FVector& Bottom, const FVector::FReal InRadius, const FColor& Color, const FString& Text /*= FString()*/)
	{
		if (!Context.ShouldLogEntity())
		{
			return;
		}

		const float Radius = static_cast<float>(InRadius);
		UE_VLOG_CIRCLE_THICK(Context.GetLogOwner(), Context.Category, Log, Bottom, FVector(0,0,1), Radius, Color, /*Thickness*/2, TEXT("%s"), *Text);

		if (UseDrawDebugHelper() && Context.World)
		{
			DrawDebugCircle(Context.World, Bottom, InRadius, /*Segments = */24, Color);
		}
	}
} // UE::MassNavigation::Debug

#endif //WITH_MASSGAMEPLAY_DEBUG
