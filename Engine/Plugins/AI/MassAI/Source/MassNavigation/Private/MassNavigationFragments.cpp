// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavigationFragments.h"
#include "GameFramework/GameStateBase.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavigationFragments)

const float FMassMoveTargetFragment::UnsetDistance = -1.f;

void FMassMoveTargetFragment::CreateNewAction(const EMassMovementAction InAction, const UWorld& InWorld)
{
	ensureMsgf(InWorld.GetNetMode() != NM_Client, TEXT("This version of SetDesiredAction should only be called on the authority."));
	CurrentAction = InAction;
	CurrentActionID++;
	CurrentActionWorldStartTime = InWorld.TimeSeconds;

	const AGameStateBase* GameState = InWorld.GetGameState();
	CurrentActionServerStartTime = GameState != nullptr ? GameState->GetServerWorldTimeSeconds() : CurrentActionWorldStartTime;

	MarkNetDirty();
}

void FMassMoveTargetFragment::CreateReplicatedAction(const EMassMovementAction InAction, const uint16 InActionID, const double InWorldStartTime, const double InServerStartTime)
{
	CurrentAction = InAction;
	CurrentActionID = InActionID;
	CurrentActionWorldStartTime = InWorldStartTime;
	CurrentActionServerStartTime = InServerStartTime;
}

FString FMassMoveTargetFragment::ToString() const
{
	return FString::Printf(TEXT("ActionID:%d Action:%s StartTime: World:%.1f Server:%.1f DesiredSpeed:%.1f "),
		CurrentActionID,
		*UEnum::GetValueAsString(CurrentAction),
		CurrentActionWorldStartTime,
		CurrentActionServerStartTime,
		DesiredSpeed.Get());
}
