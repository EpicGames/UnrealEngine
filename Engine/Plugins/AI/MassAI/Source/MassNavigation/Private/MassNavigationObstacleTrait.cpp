// Copyright Epic Games, Inc. All Rights Reserved.
#include "Avoidance/MassNavigationObstacleTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassNavigationFragments.h"
#include "MassCommonFragments.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavigationObstacleTrait)

void UMassNavigationObstacleTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.RequireFragment<FAgentRadiusFragment>();

	BuildContext.AddFragment<FMassNavigationObstacleGridCellLocationFragment>();
}
