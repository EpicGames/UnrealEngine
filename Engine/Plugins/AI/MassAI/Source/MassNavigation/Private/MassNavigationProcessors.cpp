// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavigationProcessors.h"
#include "MassCommonUtils.h"
#include "MassCommandBuffer.h"
#include "MassCommonFragments.h"
#include "MassNavigationFragments.h"
#include "MassNavigationSubsystem.h"
#include "MassSimulationLOD.h"
#include "MassMovementTypes.h"
#include "MassMovementFragments.h"
#include "MassEntityView.h"
#include "MassExecutionContext.h"
#include "Engine/World.h"


//----------------------------------------------------------------------//
//  UMassOffLODNavigationProcessor
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavigationProcessors)

UMassOffLODNavigationProcessor::UMassOffLODNavigationProcessor()
	: EntityQuery_Conditional(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Movement;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Avoidance); // @todo: remove this direct dependency
}

void UMassOffLODNavigationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery_Conditional.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery_Conditional.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	EntityQuery_Conditional.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.SetChunkFilter(&FMassSimulationVariableTickChunkFragment::ShouldTickChunkThisFrame);
}

void UMassOffLODNavigationProcessor::Execute(FMassEntityManager& EntityManager,
													FMassExecutionContext& Context)
{
	EntityQuery_Conditional.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
#if WITH_MASSGAMEPLAY_DEBUG
		if (UE::MassMovement::bFreezeMovement)
		{
			return;
		}
#endif // WITH_MASSGAMEPLAY_DEBUG

		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
		const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FTransform& CurrentTransform = LocationList[EntityIt].GetMutableTransform();
			const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];

			// Snap position to move target directly
			CurrentTransform.SetLocation(MoveTarget.Center);
		}
	});
}


//----------------------------------------------------------------------//
//  UMassNavigationSmoothHeightProcessor
//----------------------------------------------------------------------//

UMassNavigationSmoothHeightProcessor::UMassNavigationSmoothHeightProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
}

void UMassNavigationSmoothHeightProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddConstSharedRequirement<FMassMovementParameters>(EMassFragmentPresence::All);
}

void UMassNavigationSmoothHeightProcessor::Execute(FMassEntityManager& EntityManager,
													FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
#if WITH_MASSGAMEPLAY_DEBUG
		if (UE::MassMovement::bFreezeMovement)
		{
			return;
		}
#endif // WITH_MASSGAMEPLAY_DEBUG
		const float DeltaTime = Context.GetDeltaTimeSeconds();

		const FMassMovementParameters& MovementParams = Context.GetConstSharedFragment<FMassMovementParameters>();
		const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
		const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FTransform& CurrentTransform = LocationList[EntityIt].GetMutableTransform();
			const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];

			if (MoveTarget.GetCurrentAction() == EMassMovementAction::Move || MoveTarget.GetCurrentAction() == EMassMovementAction::Stand)
			{
				// Set height smoothly to follow current move targets height.
				FVector CurrentLocation = CurrentTransform.GetLocation();
				FMath::ExponentialSmoothingApprox(CurrentLocation.Z, MoveTarget.Center.Z, DeltaTime, MovementParams.HeightSmoothingTime);
				CurrentTransform.SetLocation(CurrentLocation);
			}
		}
	});
}

//----------------------------------------------------------------------//
//  UMassMoveTargetFragmentInitializer
//----------------------------------------------------------------------//

UMassMoveTargetFragmentInitializer::UMassMoveTargetFragmentInitializer()
	: InitializerQuery(*this)
{
	ObservedTypes.Add(FMassMoveTargetFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
}

void UMassMoveTargetFragmentInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	InitializerQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadWrite);
	InitializerQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
}

void UMassMoveTargetFragmentInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	InitializerQuery.ForEachEntityChunk(Context, [](FMassExecutionContext& Context)
	{
		const TArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetMutableFragmentView<FMassMoveTargetFragment>();
		const TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
			const FTransformFragment& Location = LocationList[EntityIt];

			MoveTarget.Center = Location.GetTransform().GetLocation();
			MoveTarget.Forward = Location.GetTransform().GetRotation().Vector();
			MoveTarget.DistanceToGoal = 0.0f;
			MoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
			MoveTarget.SlackRadius = 0.0f;
		}
	});
}


//----------------------------------------------------------------------//
//  UMassNavigationObstacleGridProcessor
//----------------------------------------------------------------------//
UMassNavigationObstacleGridProcessor::UMassNavigationObstacleGridProcessor()
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Movement);
}

void UMassNavigationObstacleGridProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	FMassEntityQuery BaseEntityQuery(EntityManager);
	BaseEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	BaseEntityQuery.AddRequirement<FAgentRadiusFragment>(EMassFragmentAccess::ReadOnly);
	BaseEntityQuery.AddRequirement<FMassNavigationObstacleGridCellLocationFragment>(EMassFragmentAccess::ReadWrite);
	BaseEntityQuery.AddSubsystemRequirement<UMassNavigationSubsystem>(EMassFragmentAccess::ReadWrite);

	AddToGridEntityQuery = BaseEntityQuery;
	AddToGridEntityQuery.AddRequirement<FMassAvoidanceColliderFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	AddToGridEntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	AddToGridEntityQuery.AddTagRequirement<FMassInNavigationObstacleGridTag>(EMassFragmentPresence::None);
	AddToGridEntityQuery.RegisterWithProcessor(*this);

	UpdateGridEntityQuery = BaseEntityQuery;
	UpdateGridEntityQuery.AddRequirement<FMassAvoidanceColliderFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	UpdateGridEntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	UpdateGridEntityQuery.AddTagRequirement<FMassInNavigationObstacleGridTag>(EMassFragmentPresence::All);
	UpdateGridEntityQuery.RegisterWithProcessor(*this);

	RemoveFromGridEntityQuery = BaseEntityQuery;
	RemoveFromGridEntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	RemoveFromGridEntityQuery.AddTagRequirement<FMassInNavigationObstacleGridTag>(EMassFragmentPresence::All);
	RemoveFromGridEntityQuery.RegisterWithProcessor(*this);
}

void UMassNavigationObstacleGridProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	// can't be ParallelFor due to MovementSubsystem->GetGridMutable().Move not being thread-safe
	AddToGridEntityQuery.ForEachEntityChunk(Context, [this, &EntityManager](FMassExecutionContext& Context)
	{
		FNavigationObstacleHashGrid2D& HashGrid = Context.GetMutableSubsystemChecked<UMassNavigationSubsystem>().GetObstacleGridMutable();

		TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		TConstArrayView<FAgentRadiusFragment> RadiiList = Context.GetFragmentView<FAgentRadiusFragment>();
		TArrayView<FMassNavigationObstacleGridCellLocationFragment> NavigationObstacleCellLocationList = Context.GetMutableFragmentView<FMassNavigationObstacleGridCellLocationFragment>();
		const bool bHasColliderData = Context.GetFragmentView<FMassAvoidanceColliderFragment>().Num() > 0;

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// Add to the grid
			const FVector NewPos = LocationList[EntityIt].GetTransform().GetLocation();
			const float Radius = RadiiList[EntityIt].Radius;

			FMassNavigationObstacleItem ObstacleItem;
			ObstacleItem.Entity = Context.GetEntity(EntityIt);
			ObstacleItem.ItemFlags |= bHasColliderData ? EMassNavigationObstacleFlags::HasColliderData : EMassNavigationObstacleFlags::None;
			
			const FBox NewBounds(NewPos - FVector(Radius, Radius, 0.f), NewPos + FVector(Radius, Radius, 0.f));
			NavigationObstacleCellLocationList[EntityIt].CellLoc = HashGrid.Add(ObstacleItem, NewBounds);

			Context.Defer().AddTag<FMassInNavigationObstacleGridTag>(ObstacleItem.Entity);
		}
	});

	UpdateGridEntityQuery.ForEachEntityChunk(Context, [this, &EntityManager](FMassExecutionContext& Context)
	{
		FNavigationObstacleHashGrid2D& HashGrid = Context.GetMutableSubsystemChecked<UMassNavigationSubsystem>().GetObstacleGridMutable();

		TConstArrayView<FTransformFragment> LocationList = Context.GetFragmentView<FTransformFragment>();
		TConstArrayView<FAgentRadiusFragment> RadiiList = Context.GetFragmentView<FAgentRadiusFragment>();
		TArrayView<FMassNavigationObstacleGridCellLocationFragment> NavigationObstacleCellLocationList = Context.GetMutableFragmentView<FMassNavigationObstacleGridCellLocationFragment>();
		const bool bHasColliderData = Context.GetFragmentView<FMassAvoidanceColliderFragment>().Num() > 0;

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			// Update position in grid
			const FVector NewPos = LocationList[EntityIt].GetTransform().GetLocation();
			const float Radius = RadiiList[EntityIt].Radius;
			FMassNavigationObstacleItem ObstacleItem;
			ObstacleItem.Entity = Context.GetEntity(EntityIt);
			ObstacleItem.ItemFlags |= bHasColliderData ? EMassNavigationObstacleFlags::HasColliderData : EMassNavigationObstacleFlags::None;

			const FBox NewBounds(NewPos - FVector(Radius, Radius, 0.f), NewPos + FVector(Radius, Radius, 0.f));
			NavigationObstacleCellLocationList[EntityIt].CellLoc = HashGrid.Move(ObstacleItem, NavigationObstacleCellLocationList[EntityIt].CellLoc, NewBounds);

#if WITH_MASSGAMEPLAY_DEBUG && 0
			const FDebugContext BaseDebugContext(this, LogAvoidance, nullptr, ObstacleItem.Entity);
			if (DebugIsSelected(ObstacleItem.Entity))
			{
				FBox Box = MovementSubsystem->GetGridMutable().CalcCellBounds(AvoidanceObstacleCellLocationList[EntityIt].CellLoc);
				Box.Max.Z += 200.f;
				DebugDrawBox(BaseDebugContext, Box, FColor::Yellow);
			}
#endif // WITH_MASSGAMEPLAY_DEBUG
		}
	});

	RemoveFromGridEntityQuery.ForEachEntityChunk(Context, [this, &EntityManager](FMassExecutionContext& Context)
	{
		FNavigationObstacleHashGrid2D& HashGrid = Context.GetMutableSubsystemChecked<UMassNavigationSubsystem>().GetObstacleGridMutable();

		TArrayView<FMassNavigationObstacleGridCellLocationFragment> AvoidanceObstacleCellLocationList = Context.GetMutableFragmentView<FMassNavigationObstacleGridCellLocationFragment>();
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassNavigationObstacleItem ObstacleItem;
			ObstacleItem.Entity = Context.GetEntity(EntityIt);
			HashGrid.Remove(ObstacleItem, AvoidanceObstacleCellLocationList[EntityIt].CellLoc);
			AvoidanceObstacleCellLocationList[EntityIt].CellLoc = FNavigationObstacleHashGrid2D::FCellLocation();

			Context.Defer().RemoveTag<FMassInNavigationObstacleGridTag>(ObstacleItem.Entity);
		}
	});
}

//----------------------------------------------------------------------//
//  UMassNavigationObstacleRemoverProcessor
//----------------------------------------------------------------------//
UMassNavigationObstacleRemoverProcessor::UMassNavigationObstacleRemoverProcessor()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassNavigationObstacleGridCellLocationFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Remove;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
}

void UMassNavigationObstacleRemoverProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassNavigationObstacleGridCellLocationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddSubsystemRequirement<UMassNavigationSubsystem>(EMassFragmentAccess::ReadWrite);
}

void UMassNavigationObstacleRemoverProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		FNavigationObstacleHashGrid2D& HashGrid = Context.GetMutableSubsystemChecked<UMassNavigationSubsystem>().GetObstacleGridMutable();
		const TArrayView<FMassNavigationObstacleGridCellLocationFragment> AvoidanceObstacleCellLocationList = Context.GetMutableFragmentView<FMassNavigationObstacleGridCellLocationFragment>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassNavigationObstacleItem ObstacleItem;
			ObstacleItem.Entity = Context.GetEntity(EntityIt);
			HashGrid.Remove(ObstacleItem, AvoidanceObstacleCellLocationList[EntityIt].CellLoc);
		}
	});
}
