// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavigationSubsystem.h"
#include "Engine/World.h"
#include "MassSimulationSubsystem.h"

//----------------------------------------------------------------------//
// UMassNavigationSubsystem
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavigationSubsystem)
UMassNavigationSubsystem::UMassNavigationSubsystem()
	: AvoidanceObstacleGrid(250.f) // 2.5m grid
{
}

void UMassNavigationSubsystem::Initialize(FSubsystemCollectionBase& Collection)
{
	Super::Initialize(Collection);
	Collection.InitializeDependency<UMassSimulationSubsystem>();

	OverrideSubsystemTraits<UMassNavigationSubsystem>(Collection);
}

