// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavigationTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavigationTypes)

DEFINE_LOG_CATEGORY(LogMassNavigation);
DEFINE_LOG_CATEGORY(LogMassNavigationObstacle);
