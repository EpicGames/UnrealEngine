// Copyright Epic Games, Inc. All Rights Reserved.

#include "SmoothOrientation/MassSmoothOrientationProcessors.h"
#include "SmoothOrientation/MassSmoothOrientationFragments.h"
#include "MassCommandBuffer.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassNavigationFragments.h"
#include "MassMovementFragments.h"
#include "Math/UnrealMathUtility.h"
#include "MassSimulationLOD.h"
#include "MassNavigationUtils.h"


//----------------------------------------------------------------------//
//  UMassSmoothOrientationProcessor
//----------------------------------------------------------------------//

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmoothOrientationProcessors)
UMassSmoothOrientationProcessor::UMassSmoothOrientationProcessor()
	: HighResEntityQuery(*this)
	, LowResEntityQuery_Conditional(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Movement;
}

void UMassSmoothOrientationProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	HighResEntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	HighResEntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadWrite);
	HighResEntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	HighResEntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	HighResEntityQuery.AddConstSharedRequirement<FMassSmoothOrientationParameters>(EMassFragmentPresence::All);

	LowResEntityQuery_Conditional.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadWrite);
	LowResEntityQuery_Conditional.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	LowResEntityQuery_Conditional.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::All);
	LowResEntityQuery_Conditional.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	LowResEntityQuery_Conditional.SetChunkFilter(&FMassSimulationVariableTickChunkFragment::ShouldTickChunkThisFrame);
}

void UMassSmoothOrientationProcessor::Execute(FMassEntityManager& EntityManager,
													FMassExecutionContext& Context)
{
	// Clamp max delta time to avoid force explosion on large time steps (i.e. during initialization).
	const float DeltaTime = FMath::Min(0.1f, Context.GetDeltaTimeSeconds());

	{
		QUICK_SCOPE_CYCLE_COUNTER(HighRes);

		HighResEntityQuery.ForEachEntityChunk(Context, [this, DeltaTime](FMassExecutionContext& Context)
		{
			const FMassSmoothOrientationParameters& OrientationParams = Context.GetConstSharedFragment<FMassSmoothOrientationParameters>();

			const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();
			const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
			const TArrayView<FMassDesiredMovementFragment> DesiredMovementList = Context.GetMutableFragmentView<FMassDesiredMovementFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];

				// Do not touch transform at all when animating
				if (MoveTarget.GetCurrentAction() == EMassMovementAction::Animate)
				{
					continue;
				}

				FMassDesiredMovementFragment& DesiredMovement = DesiredMovementList[EntityIt];
				FTransform& CurrentTransform = LocationList[EntityIt].GetMutableTransform();
				const FVector CurrentForward = CurrentTransform.GetRotation().GetForwardVector();
				const FVector::FReal CurrentHeading = UE::MassNavigation::GetYawFromDirection(CurrentForward);

				const float EndOfPathAnticipationDistance = OrientationParams.EndOfPathDuration * MoveTarget.DesiredSpeed.Get();
				
				FVector::FReal MoveTargetWeight = 0.5;
				FVector::FReal VelocityWeight = 0.5;
				
				if (MoveTarget.GetCurrentAction() == EMassMovementAction::Move)
				{
					if (MoveTarget.IntentAtGoal == EMassMovementAction::Stand && MoveTarget.DistanceToGoal < EndOfPathAnticipationDistance)
					{
						// Fade towards the movement target direction at the end of the path.
						const float Fade = FMath::Square(FMath::Clamp(MoveTarget.DistanceToGoal / EndOfPathAnticipationDistance, 0.0f, 1.0f)); // zero at end of the path

						MoveTargetWeight = FMath::Lerp(OrientationParams.Standing.MoveTargetWeight, OrientationParams.Moving.MoveTargetWeight, Fade);
						VelocityWeight = FMath::Lerp(OrientationParams.Standing.VelocityWeight, OrientationParams.Moving.VelocityWeight, Fade);
					}
					else
					{
						MoveTargetWeight = OrientationParams.Moving.MoveTargetWeight;
						VelocityWeight = OrientationParams.Moving.VelocityWeight;
					}
				}
				else // Stand
				{
					MoveTargetWeight = OrientationParams.Standing.MoveTargetWeight;
					VelocityWeight = OrientationParams.Standing.VelocityWeight;
				}
				
				const FVector::FReal VelocityHeading = UE::MassNavigation::GetYawFromDirection(DesiredMovement.DesiredVelocity);
				const FVector::FReal MovementHeading = UE::MassNavigation::GetYawFromDirection(MoveTarget.Forward);

				const FVector::FReal Ratio = MoveTargetWeight / (MoveTargetWeight + VelocityWeight);
				const FVector::FReal DesiredHeading = UE::MassNavigation::LerpAngle(VelocityHeading, MovementHeading,Ratio);
				DesiredMovement.DesiredFacing = FQuat(FVector::UpVector, DesiredHeading);
				
				const FVector::FReal NewHeading = UE::MassNavigation::ExponentialSmoothingAngle(CurrentHeading, DesiredHeading, DeltaTime, OrientationParams.OrientationSmoothingTime);

				FQuat Rotation(FVector::UpVector, NewHeading);
				CurrentTransform.SetRotation(Rotation);
			}
		});
	}

	{
		QUICK_SCOPE_CYCLE_COUNTER(LowRes);

		LowResEntityQuery_Conditional.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
		{
			const TArrayView<FTransformFragment> LocationList = Context.GetMutableFragmentView<FTransformFragment>();
			const TConstArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();

			for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
			{
				FTransform& CurrentTransform = LocationList[EntityIt].GetMutableTransform();
				const FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];

				// Snap position to move target directly
				CurrentTransform.SetRotation(FQuat::FindBetweenNormals(FVector::ForwardVector, MoveTarget.Forward));
			}
		});
	}
}
