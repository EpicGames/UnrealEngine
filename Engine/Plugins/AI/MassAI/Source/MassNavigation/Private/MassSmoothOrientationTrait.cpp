// Copyright Epic Games, Inc. All Rights Reserved.
#include "SmoothOrientation/MassSmoothOrientationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassMovementFragments.h"
#include "MassCommonFragments.h"
#include "MassNavigationFragments.h"
#include "Engine/World.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSmoothOrientationTrait)


void UMassSmoothOrientationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	BuildContext.RequireFragment<FMassMoveTargetFragment>();
	BuildContext.RequireFragment<FMassVelocityFragment>();
	BuildContext.RequireFragment<FTransformFragment>();

	const FConstSharedStruct OrientationFragment = EntityManager.GetOrCreateConstSharedFragment(Orientation);
	BuildContext.AddConstSharedFragment(OrientationFragment);
}
