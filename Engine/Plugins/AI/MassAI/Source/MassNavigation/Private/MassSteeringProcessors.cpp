// Copyright Epic Games, Inc. All Rights Reserved.

#include "Steering/MassSteeringProcessors.h"
#include "MassCommonUtils.h"
#include "MassCommandBuffer.h"
#include "MassDebugLogging.h"
#include "MassCommonFragments.h"
#include "MassDebugger.h"
#include "MassExecutionContext.h"
#include "MassMovementFragments.h"
#include "MassNavigationFragments.h"
#include "Steering/MassSteeringFragments.h"
#include "Math/UnrealMathUtility.h"
#include "MassSimulationLOD.h"
#if WITH_MASSGAMEPLAY_DEBUG
#include "MassNavigationDebug.h"
#endif // WITH_MASSGAMEPLAY_DEBUG

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSteeringProcessors)

namespace UE::MassNavigation
{
	/*
	* Calculates speed scale based on agent's forward direction and desired steering direction.
	*/
	static FVector::FReal CalcDirectionalSpeedScale(const FVector ForwardDirection, const FVector SteerDirection)
	{
		// @todo: make these configurable
		constexpr FVector::FReal ForwardSpeedScale = 1.;
		constexpr FVector::FReal BackwardSpeedScale = 0.25;
		constexpr FVector::FReal SideSpeedScale = 0.5;

		const FVector LeftDirection = FVector::CrossProduct(ForwardDirection, FVector::UpVector);
		const FVector::FReal DirX = FVector::DotProduct(LeftDirection, SteerDirection);
		const FVector::FReal DirY = FVector::DotProduct(ForwardDirection, SteerDirection);

		// Calculate intersection between a direction vector and ellipse, where A & B are the size of the ellipse.
		// The direction vector is starting from the center of the ellipse.
		constexpr FVector::FReal SideA = SideSpeedScale;
		const FVector::FReal SideB = DirY > 0. ? ForwardSpeedScale : BackwardSpeedScale;
		const FVector::FReal Disc = FMath::Square(SideA) * FMath::Square(DirY) + FMath::Square(SideB) * FMath::Square(DirX);
		const FVector::FReal Speed = (Disc > SMALL_NUMBER) ? (SideA * SideB / FMath::Sqrt(Disc)) : 0.;

		return Speed;
	}

	/** Speed envelope when approaching a point. NormalizedDistance in range [0..1] */
	static FVector::FReal ArrivalSpeedEnvelope(const FVector::FReal NormalizedDistance)
	{
		return FMath::Sqrt(NormalizedDistance);
	}

} // UE::MassNavigation

//----------------------------------------------------------------------//
//  UMassSteerToMoveTargetProcessor
//----------------------------------------------------------------------//
UMassSteerToMoveTargetProcessor::UMassSteerToMoveTargetProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = int32(EProcessorExecutionFlags::AllNetModes);
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::Tasks);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Avoidance);
}

void UMassSteerToMoveTargetProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassVelocityFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassSteeringFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassStandingSteeringFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassGhostLocationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassForceFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassDesiredMovementFragment>(EMassFragmentAccess::ReadWrite);	// Note this is read-write because this processor will sometimes zero the desired velocity, but normally it affects it via the FForceFragment
	EntityQuery.AddConstSharedRequirement<FMassMovementParameters>(EMassFragmentPresence::All);
	EntityQuery.AddConstSharedRequirement<FMassMovingSteeringParameters>(EMassFragmentPresence::All);
	EntityQuery.AddConstSharedRequirement<FMassStandingSteeringParameters>(EMassFragmentPresence::All);
	EntityQuery.AddTagRequirement<FMassSteerToMoveTargetPreventSlowdownTag>(EMassFragmentPresence::Optional);
	
#if WITH_MASSGAMEPLAY_DEBUG
	EntityQuery.DebugEnableEntityOwnerLogging();
#endif

	// No need for Off LOD to do steering, applying move target directly
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);
	EntityQuery.AddTagRequirement<FMassCodeDrivenMovementTag>(EMassFragmentPresence::Optional);
}

void UMassSteerToMoveTargetProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	UWorld* DebugDrawWorld = UE::Mass::Debug::GetDebugDrawWorld(Context);
	
	EntityQuery.ForEachEntityChunk(Context, [FallbackLogOwner = this, DebugDrawWorld](FMassExecutionContext& Context)
	{
		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
		const TConstArrayView<FMassVelocityFragment> VelocityList = Context.GetFragmentView<FMassVelocityFragment>();
		const TArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetMutableFragmentView<FMassMoveTargetFragment>();
		const TArrayView<FMassDesiredMovementFragment> MovementList = Context.GetMutableFragmentView<FMassDesiredMovementFragment>();
		const TArrayView<FMassForceFragment> ForceList = Context.GetMutableFragmentView<FMassForceFragment>();
		const TArrayView<FMassSteeringFragment> SteeringList = Context.GetMutableFragmentView<FMassSteeringFragment>();
		const TArrayView<FMassStandingSteeringFragment> StandingSteeringList = Context.GetMutableFragmentView<FMassStandingSteeringFragment>();
		const TArrayView<FMassGhostLocationFragment> GhostList = Context.GetMutableFragmentView<FMassGhostLocationFragment>();
		const FMassMovementParameters& MovementParams = Context.GetConstSharedFragment<FMassMovementParameters>();
		const FMassMovingSteeringParameters& MovingSteeringParams = Context.GetConstSharedFragment<FMassMovingSteeringParameters>();
		const FMassStandingSteeringParameters& StandingSteeringParams = Context.GetConstSharedFragment<FMassStandingSteeringParameters>();
		const bool bIsCodeDriven = Context.DoesArchetypeHaveTag<FMassCodeDrivenMovementTag>();
		const bool bPreventSteeringSlowdown = Context.DoesArchetypeHaveTag<FMassSteerToMoveTargetPreventSlowdownTag>();

		const FVector::FReal SteerK = 1. / MovingSteeringParams.ReactionTime;
		const float DeltaTime = Context.GetDeltaTimeSeconds();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FTransformFragment& TransformFragment = TransformList[EntityIt];
			const FMassVelocityFragment& VelocityFragment = VelocityList[EntityIt];
			FMassSteeringFragment& Steering = SteeringList[EntityIt];
			FMassStandingSteeringFragment& StandingSteering = StandingSteeringList[EntityIt];
			FMassGhostLocationFragment& Ghost = GhostList[EntityIt];
			FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
			FMassForceFragment& Force = ForceList[EntityIt];
			FMassDesiredMovementFragment& DesiredMovement = MovementList[EntityIt];
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);

			const FTransform& Transform = TransformFragment.GetTransform();;

			FVector::FReal SlowdownScale = 1;

			// Calculate velocity for steering.
			const FVector CurrentLocation = Transform.GetLocation();
			const FVector CurrentForward = Transform.GetRotation().GetForwardVector();

			const FVector::FReal LookAheadDistance = FMath::Max(1.0f, MovingSteeringParams.LookAheadTime * MoveTarget.DesiredSpeed.Get());

#if WITH_MASSGAMEPLAY_DEBUG
			UE::MassNavigation::Debug::FDebugContext NavigationDebugContext(Context, FallbackLogOwner, LogMassNavigation, DebugDrawWorld, Entity, EntityIt); 
			const bool bDisplayDebug = NavigationDebugContext.ShouldLogEntity();
			const UObject* LogOwner = NavigationDebugContext.GetLogOwner();
#endif

			if (MoveTarget.GetCurrentAction() == EMassMovementAction::Move)
			{
				// Tune down avoidance and speed when arriving at goal.
				FVector::FReal ArrivalFade = 1.;
				if (MoveTarget.IntentAtGoal == EMassMovementAction::Stand)
				{
					ArrivalFade = FMath::Clamp(MoveTarget.DistanceToGoal / LookAheadDistance, 0., 1.);
				}
				const FVector::FReal SteeringPredictionDistance = LookAheadDistance * ArrivalFade;

				// Steer towards and along the move target.
				const FVector TargetSide = FVector::CrossProduct(MoveTarget.Forward, FVector::UpVector);
				const FVector Delta = CurrentLocation - MoveTarget.Center;

				const FVector::FReal ForwardOffset = FVector::DotProduct(MoveTarget.Forward, Delta);

				// Calculate steering direction. When far away from the line defined by TargetPosition and TargetTangent,
				// the steering direction is towards the line, the close we get, the more it aligns with the line.
				const FVector::FReal SidewaysOffset = FVector::DotProduct(TargetSide, Delta);
				const FVector::FReal SteerForward = FMath::Sqrt(FMath::Max(0., FMath::Square(SteeringPredictionDistance) - FMath::Square(SidewaysOffset)));

				// The Max() here makes the steering directions behind the TargetPosition to steer towards it directly.
				FVector SteerTarget = MoveTarget.Center + MoveTarget.Forward * FMath::Clamp(ForwardOffset + SteerForward, 0., SteeringPredictionDistance);

				FVector SteerDirection = SteerTarget - CurrentLocation;
				SteerDirection.Z = 0.;
				const FVector::FReal DistanceToSteerTarget = SteerDirection.Length();
				if (DistanceToSteerTarget > KINDA_SMALL_NUMBER)
				{
					SteerDirection *= 1. / DistanceToSteerTarget;
				}

#if WITH_MASSGAMEPLAY_DEBUG
				if (bDisplayDebug)
				{
					// Display SteerDirection
					const FVector ZOffset(0,0,25);
					UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Log, CurrentLocation + ZOffset, CurrentLocation + ZOffset + 100.*SteerDirection,
						FColor::Red, /*Thickness*/2, TEXT("SteerDirection"));
				}
#endif //WITH_MASSGAMEPLAY_DEBUG

				FVector::FReal DesiredSpeed = MoveTarget.DesiredSpeed.Get();

				// When being animation driven, animation has authority over the movement, so it might be useful to disable
				// this catchup mechanic to avoid subtle speed variations affecting animation.
				if (MovingSteeringParams.bAllowSpeedVariance)
				{
					const FVector::FReal DirSpeedScale = UE::MassNavigation::CalcDirectionalSpeedScale(CurrentForward, SteerDirection);
					DesiredSpeed *= DirSpeedScale;

					// Control speed based relation to the forward axis of the move target.
					FVector::FReal CatchupDesiredSpeed = DesiredSpeed;
					if (ForwardOffset < 0.)
					{
						// Falling behind, catch up
						const FVector::FReal T = FMath::Min(-ForwardOffset / LookAheadDistance, 1.);
						CatchupDesiredSpeed = FMath::Lerp(DesiredSpeed, MovementParams.MaxSpeed, T);
					}
					else if (ForwardOffset > 0.)
					{
						// Ahead, slow down.
						const FVector::FReal T = FMath::Min(ForwardOffset / LookAheadDistance, 1.);
						CatchupDesiredSpeed = FMath::Lerp(DesiredSpeed, DesiredSpeed * 0., 1. - FMath::Square(1. - T));
					}

					// Control speed based on distance to move target. This allows to catch up even if speed above reaches zero.
					const FVector::FReal DeviantSpeed = FMath::Min(FMath::Abs(SidewaysOffset) / LookAheadDistance, 1.) * DesiredSpeed;

					DesiredSpeed = FMath::Max(CatchupDesiredSpeed, DeviantSpeed);
				}

				// Slow down towards the end of path.
				if (MoveTarget.IntentAtGoal == EMassMovementAction::Stand)
				{
					const FVector::FReal NormalizedDistanceToSteerTarget = FMath::Clamp(DistanceToSteerTarget / LookAheadDistance, 0., 1.);
					DesiredSpeed *= UE::MassNavigation::ArrivalSpeedEnvelope(FMath::Max(ArrivalFade, NormalizedDistanceToSteerTarget));
				}

				constexpr FVector::FReal FallingBehindScale = 0.8;
				if (MoveTarget.EntityDistanceToGoal != FMassMoveTargetFragment::UnsetDistance)
				{
					MoveTarget.bSteeringFallingBehind = (MoveTarget.EntityDistanceToGoal - MoveTarget.DistanceToGoal) > LookAheadDistance * FallingBehindScale;	
				}
				else
				{
					// If EntityDistanceToGoal is not available, use ForwardOffset. 
					MoveTarget.bSteeringFallingBehind = ForwardOffset < -LookAheadDistance * FallingBehindScale;
				}

				// @todo: This current completely overrides steering, we probably should have one processor that resets the steering at the beginning of the frame.
				Steering.DesiredVelocity = SteerDirection * DesiredSpeed;

				// Important: we want steering force to be stable against noisy velocity, so we use
				// the difference between current desired velocity and target desired velocity.
				// We dont want to read the actual agent velocity directly since this can create a feedback loop for animated characters
				Force.Value = SteerK * (Steering.DesiredVelocity - DesiredMovement.DesiredVelocity); // Goal force

				// Apply only if DesiredVelocity is above or equal to DefaultDesiredSpeed, else we want the steering force to slow us down to the desired speed.
				if (bPreventSteeringSlowdown && DesiredMovement.DesiredVelocity.Size() <= MovementParams.DefaultDesiredSpeed)
				{
					const FVector::FReal DistanceToMoveTarget = FVector::Dist(CurrentLocation, MoveTarget.Center);
					const FVector::FReal ExtraDistance = FMath::Max(0, DistanceToMoveTarget - (1.1*LookAheadDistance)); // Multiplied by 1.1 to add some buffer
					SlowdownScale = 1 - FMath::Clamp(ExtraDistance/MovingSteeringParams.SteeringPreventSlowdownAttenuationDistance, 0, 1);

					// Make sure the steering force does not reduce DesiredMovement.DesiredVelocity.
					const FVector DesiredVelocityNormal = DesiredMovement.DesiredVelocity.GetSafeNormal();
					const FVector::FReal ProjectedForceSize = Force.Value.Dot(DesiredVelocityNormal);
					if (ProjectedForceSize < 0)
					{
						// The force is slowing the entity, avoid that.
						Force.Value += SlowdownScale * FMath::Abs(ProjectedForceSize) * DesiredVelocityNormal;
					}
				}
			}
			else if (MoveTarget.GetCurrentAction() == EMassMovementAction::Stand)
			{
				// Calculate unique target move threshold so that different agents react a bit differently.
				const FVector::FReal PerEntityScale = UE::RandomSequence::FRand(Entity.Index);
				const FVector::FReal TargetMoveThreshold = StandingSteeringParams.TargetMoveThreshold * (1. - StandingSteeringParams.TargetMoveThresholdVariance + PerEntityScale * StandingSteeringParams.TargetMoveThresholdVariance * 2.);
				
				if (Ghost.LastSeenActionID != MoveTarget.GetCurrentActionID())
				{
					// Reset when action changes. @todo: should reset only when move->stand?
					Ghost.Location = MoveTarget.Center;
					Ghost.Velocity = FVector::ZeroVector;
					Ghost.LastSeenActionID = MoveTarget.GetCurrentActionID();

					StandingSteering.TargetLocation = MoveTarget.Center;
					StandingSteering.TrackedTargetSpeed = 0.0f;
					StandingSteering.bIsUpdatingTarget = false;
					StandingSteering.TargetSelectionCooldown = StandingSteeringParams.TargetSelectionCooldown * FMath::RandRange(1.f - StandingSteeringParams.TargetSelectionCooldownVariance, 1.f + StandingSteeringParams.TargetSelectionCooldownVariance);
					StandingSteering.bEnteredFromMoveAction = MoveTarget.GetPreviousAction() == EMassMovementAction::Move;
				}

				StandingSteering.TargetSelectionCooldown = FMath::Max(0.0f, StandingSteering.TargetSelectionCooldown - DeltaTime);

				if (!StandingSteering.bIsUpdatingTarget)
				{
					// Update the move target if enough time has passed and the target has moved. 
					if (StandingSteering.TargetSelectionCooldown <= 0.0f
						&& FVector::DistSquared(StandingSteering.TargetLocation, Ghost.Location) > FMath::Square(TargetMoveThreshold))
					{
						StandingSteering.TargetLocation = Ghost.Location;
						StandingSteering.TrackedTargetSpeed = 0.0f;
						StandingSteering.bIsUpdatingTarget = true;
						StandingSteering.bEnteredFromMoveAction = false;
					}
				}
				else
				{
					// Updating target
					StandingSteering.TargetLocation = Ghost.Location;
					const FVector::FReal GhostSpeed = Ghost.Velocity.Length();

					if (GhostSpeed > (StandingSteering.TrackedTargetSpeed * StandingSteeringParams.TargetSpeedHysteresisScale))
					{
						const FVector::FReal TrackedTargetSpeed = FMath::Max(StandingSteering.TrackedTargetSpeed, GhostSpeed);
						StandingSteering.TrackedTargetSpeed = static_cast<float>(TrackedTargetSpeed);
					}
					else
					{
						// Speed is dropping, we have found the peak change, stop updating the target and start cooldown.
						StandingSteering.TargetSelectionCooldown = StandingSteeringParams.TargetSelectionCooldown * FMath::RandRange(1.0f - StandingSteeringParams.TargetSelectionCooldownVariance, 1.0f + StandingSteeringParams.TargetSelectionCooldownVariance);
						StandingSteering.bIsUpdatingTarget = false;
					}
				}
				
				// Move directly towards the move target when standing.
				FVector SteerDirection = FVector::ZeroVector;
				FVector::FReal DesiredSpeed = 0.;

				FVector Delta = StandingSteering.TargetLocation - CurrentLocation;
				Delta.Z = 0.;
				const FVector::FReal Distance = Delta.Size();
				if (Distance > StandingSteeringParams.DeadZoneRadius)
				{
#if WITH_MASSGAMEPLAY_DEBUG
					if (bDisplayDebug)
					{
						UE_VLOG_UELOG(LogOwner, LogMassNavigation, Verbose, TEXT("Standing steering: out of deadzone (Distance: %.2f)"), Distance);
					}
#endif
					
					SteerDirection = Delta / Distance;
					if (StandingSteering.bEnteredFromMoveAction)
					{
						// If the current steering target is from approaching a move target, use the same speed logic as movement to ensure smooth transition.
						const FVector::FReal Range = FMath::Max(1., LookAheadDistance - StandingSteeringParams.DeadZoneRadius);
						const FVector::FReal SpeedFade = FMath::Clamp((Distance - StandingSteeringParams.DeadZoneRadius) / Range, 0., 1.);
						DesiredSpeed = MoveTarget.DesiredSpeed.Get() * UE::MassNavigation::CalcDirectionalSpeedScale(CurrentForward, SteerDirection) * UE::MassNavigation::ArrivalSpeedEnvelope(SpeedFade);
					}
					else
					{
						const FVector::FReal Range = FMath::Max(1., LookAheadDistance - StandingSteeringParams.DeadZoneRadius);
						const FVector::FReal SpeedFade = FMath::Clamp((Distance - StandingSteeringParams.DeadZoneRadius) / Range, 0., 1.);
						// Not using the directional scaling so that the steps we take to avoid are done quickly, and the behavior is reactive.
						DesiredSpeed = MoveTarget.DesiredSpeed.Get() * UE::MassNavigation::ArrivalSpeedEnvelope(SpeedFade);
					}
					
					// @todo: This current completely overrides steering, we probably should have one processor that resets the steering at the beginning of the frame.
					Steering.DesiredVelocity = SteerDirection * DesiredSpeed;
					Force.Value = SteerK * (Steering.DesiredVelocity - DesiredMovement.DesiredVelocity); // Goal force
					Force.Value = Force.Value.GetClampedToMaxSize(MovementParams.MaxAcceleration); 
				}
				else
				{
					// When reached destination, clamp small desired velocities to zero to avoid tiny drifting.
					if (DesiredMovement.DesiredVelocity.SquaredLength() < FMath::Square(StandingSteeringParams.LowSpeedThreshold))
					{
						DesiredMovement.DesiredVelocity = FVector::ZeroVector;
						Force.Value = FVector::ZeroVector;
					}
				}

				MoveTarget.bSteeringFallingBehind = false;
			}
			else if (MoveTarget.GetCurrentAction() == EMassMovementAction::Animate)
			{
				Steering.Reset();
				MoveTarget.bSteeringFallingBehind = false;
				Force.Value = FVector::ZeroVector;

				if (bIsCodeDriven)
				{
					// Stop all movement when animating.
					DesiredMovement.DesiredVelocity = FVector::ZeroVector;
				}
				else
				{
					// Animation driven, sync desired movement to animation
					Steering.DesiredVelocity = VelocityFragment.Value;
					DesiredMovement.DesiredVelocity = VelocityFragment.Value;
					if (!DesiredMovement.DesiredVelocity.IsNearlyZero())
					{
						DesiredMovement.DesiredFacing = FQuat(DesiredMovement.DesiredVelocity.Rotation());
					}
				}
			}

#if WITH_MASSGAMEPLAY_DEBUG
			FColor EntityColor = FColor::White;
			if (bDisplayDebug)
			{
				const FVector ZOffset(0,0,25);

				const FColor LightEntityColor = UE::MassNavigation::Debug::MixColors(EntityColor, FColor::White);
				
				const FVector MoveTargetCenter = MoveTarget.Center + ZOffset;

				// Display MoveTarget location
				UE_VLOG_CIRCLE_THICK(LogOwner, LogMassNavigation, Log, MoveTargetCenter, FVector::UpVector, 5.f, EntityColor, /*Thickness*/2,
					TEXT("MoveTarget\ngoal: %s"), *UEnum::GetDisplayValueAsText(MoveTarget.IntentAtGoal).ToString());

				// Display ghost location
				UE_VLOG_CIRCLE_THICK(LogOwner, LogMassNavigation, Log, Ghost.Location, FVector::UpVector, 5.f, FColor::Silver, /*Thickness*/2, TEXT("Ghost"));

				// Display MoveTarget orientation
				UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Log, MoveTargetCenter, MoveTargetCenter + MoveTarget.Forward * 100, EntityColor, /*Thickness*/1,
					TEXT("MoveTarget\nforward"));

				// Display MoveTarget - current location relation
				if (MoveTarget.DesiredSpeed.Get() > 0.f && FVector::Dist2D(CurrentLocation, MoveTarget.Center) > LookAheadDistance * 1.5f)
				{
					UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Log, MoveTargetCenter, CurrentLocation + ZOffset, FColor::Red, /*Thickness*/1, TEXT("LOST"));
				}
				else
				{
					UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Log, MoveTargetCenter, CurrentLocation + ZOffset, LightEntityColor, /*Thickness*/1, TEXT(""));
				}

				// Display DesiredMovement DesiredVelocity
				UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Log, CurrentLocation + ZOffset, CurrentLocation + ZOffset + DesiredMovement.DesiredVelocity, FColor::Yellow, /*Thickness*/4,
					TEXT("Mvt DesiredVelocity %.1f"), DesiredMovement.DesiredVelocity.Length());

				// Display Steering internal DesiredVelocity
				UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Log, CurrentLocation + ZOffset + FVector(0,0,2),
					CurrentLocation + Steering.DesiredVelocity + ZOffset + FVector(0,0,2), FColor::Orange, /*Thickness*/4,
					TEXT("Steering DesiredVelocity %.1f"), Steering.DesiredVelocity.Length());

				// Display Force
				UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Log, CurrentLocation + ZOffset, CurrentLocation + Force.Value + ZOffset, FColor::Emerald, /*Thickness*/4,
					TEXT("Steering Force %.1f"), Force.Value.Length());

				// Display slowdown scale as a bigger circle when near 0.
				constexpr FVector::FReal SlowdownScaleDisplayThreshold = 0.95; 
				if (SlowdownScale < SlowdownScaleDisplayThreshold)
				{
					UE_VLOG_CIRCLE_THICK(FallbackLogOwner, LogMassNavigation, Log, CurrentLocation, FVector::UpVector, (1-SlowdownScale)*100, FColor::Red, /*Thickness*/2,
						TEXT("Slowdown scale: %.0f"), SlowdownScale);
				}
			}
#endif // WITH_MASSGAMEPLAY_DEBUG
			
		}
	});
}
