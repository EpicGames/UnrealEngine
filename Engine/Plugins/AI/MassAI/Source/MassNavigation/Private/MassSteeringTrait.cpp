// Copyright Epic Games, Inc. All Rights Reserved.

#include "Steering/MassSteeringTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassCommonFragments.h"
#include "MassMovementFragments.h"
#include "MassNavigationFragments.h"
#include "Steering/MassSteeringFragments.h"
#include "Engine/World.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassSteeringTrait)


void UMassSteeringTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	BuildContext.RequireFragment<FAgentRadiusFragment>();
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.RequireFragment<FMassVelocityFragment>();
	BuildContext.RequireFragment<FMassForceFragment>();

	BuildContext.AddFragment<FMassMoveTargetFragment>();
	BuildContext.AddFragment<FMassSteeringFragment>();
	BuildContext.AddFragment<FMassStandingSteeringFragment>();
	BuildContext.AddFragment<FMassGhostLocationFragment>();

	const FConstSharedStruct MovingSteeringFragment = EntityManager.GetOrCreateConstSharedFragment(MovingSteering);
	BuildContext.AddConstSharedFragment(MovingSteeringFragment);

	const FConstSharedStruct StandingSteeringFragment = EntityManager.GetOrCreateConstSharedFragment(StandingSteering);
	BuildContext.AddConstSharedFragment(StandingSteeringFragment);
}
