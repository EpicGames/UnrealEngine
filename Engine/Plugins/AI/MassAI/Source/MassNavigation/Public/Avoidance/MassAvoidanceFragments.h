// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "Mass/EntityHandle.h"
#include "Mass/ExternalSubsystemTraits.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassNavigationSubsystem.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassAvoidanceFragments.generated.h"

USTRUCT()
struct FMassMovingAvoidanceParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	FMassMovingAvoidanceParameters GetValidated() const
	{
		FMassMovingAvoidanceParameters Copy = *this;
		Copy.PredictiveAvoidanceTime = FMath::Max(Copy.PredictiveAvoidanceTime, KINDA_SMALL_NUMBER);
		Copy.EnvironmentPredictiveAvoidanceTime = FMath::Max(Copy.EnvironmentPredictiveAvoidanceTime, KINDA_SMALL_NUMBER);
		Copy.ObstacleSeparationDistance = FMath::Max(Copy.ObstacleSeparationDistance, KINDA_SMALL_NUMBER);
		Copy.PredictiveAvoidanceDistance = FMath::Max(Copy.PredictiveAvoidanceDistance, KINDA_SMALL_NUMBER);
		Copy.EnvironmentPredictiveAvoidanceDistance = FMath::Max(Copy.EnvironmentPredictiveAvoidanceDistance, KINDA_SMALL_NUMBER);
		Copy.EnvironmentSeparationDistance = FMath::Max(Copy.EnvironmentSeparationDistance, KINDA_SMALL_NUMBER);
		Copy.EnvironmentSeparationBehindEdgeDistanceScale = FMath::Max(Copy.EnvironmentSeparationBehindEdgeDistanceScale, KINDA_SMALL_NUMBER);
		Copy.StartOfPathDuration = FMath::Max(Copy.StartOfPathDuration, KINDA_SMALL_NUMBER);
		Copy.EndOfPathDuration = FMath::Max(Copy.EndOfPathDuration, KINDA_SMALL_NUMBER);

		return Copy;
	}

	/** The distance at which obstacle are detected. Range: 200...600 */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0.0", ForceUnits="cm"))
	float ObstacleDetectionDistance = 400.f;

	/** The time the agent is considered to be near the start of the path when starting to move. Range: 0..3 */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0", ForceUnits="s"))
	float StartOfPathDuration = 1.0f;

	/** The time the agent is considered to be near the end of the path when approaching end. Range: 0..3 */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0", ForceUnits="s"))
	float EndOfPathDuration = 0.5f;

	/** How much to tune down the avoidance at the start of the path. Range: 0..1. */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0", ForceUnits="x"))
	float StartOfPathAvoidanceScale = 0.0f;

	/** How much to tune down the avoidance towards the end of the path. Range: 0..1 */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0", ForceUnits="x"))
	float EndOfPathAvoidanceScale = 0.1f;

	/** How much to tune down the avoidance when an obstacle is standing. This allows the agents to pass through standing agents more easily. Range: 0..1 */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0", ForceUnits="x"))
	float StandingObstacleAvoidanceScale = 0.65f;

	/** Agent radius scale for avoiding static obstacles near wall. If the clarance between obstacle and wall is less than the scaled radius, the agent will not try to move through the gap. Range: 0..1 */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0", ForceUnits="x"))
	float StaticObstacleClearanceScale = 0.7f;

	/** Agent radius scale for separation. Making it smaller makes the separation softer. Range: 0.8..1 */
	UPROPERTY(EditAnywhere, Category = "Separation", meta = (ClampMin = "0", ForceUnits="x"))
	float SeparationRadiusScale = 0.9f;

	
	/** Separation force stiffness between agents and obstacles. Range: 100..500 N/cm */
	UPROPERTY(EditAnywhere, Category = "Separation", meta = (ClampMin = "0"))
	float ObstacleSeparationStiffness = 250.f;

	/** Separation force effect distance. The actual observed separation distance will be smaller. Range: 0..100 */
	UPROPERTY(EditAnywhere, Category = "Separation", meta = (ClampMin = "0", ForceUnits="cm"))
	float ObstacleSeparationDistance = 75.f;

	/** Environment separation force stiffness between agents and walls. Range: 200..1000 N/cm */
	UPROPERTY(EditAnywhere, Category = "Separation", meta = (ClampMin = "0"))
	float EnvironmentSeparationStiffness = 500.f;

	/** Environment separation force effect distance. The actual observed separation distance will be smaller. Range: 0..200 */
	UPROPERTY(EditAnywhere, Category = "Separation", meta = (ClampMin = "0", ForceUnits="cm"))
	float EnvironmentSeparationDistance = 50.f;

	/** Environment separation distance behind edge where separation force is applied. The scale is applied to AgentRadius. Range: 1..2 */
	UPROPERTY(EditAnywhere, Category = "Separation", meta = (ClampMin = "0", ForceUnits="x"))
	float EnvironmentSeparationBehindEdgeDistanceScale = 1.f;
	
	
	/** How far in the future the agent reacts to collisions. Range: 1..3, Indoor humans 1.4, outdoor humans 2.4 (seconds). */
	UPROPERTY(EditAnywhere, Category = "Predictive Avoidance", meta = (ClampMin = "0.1", ForceUnits="s"))
	float PredictiveAvoidanceTime = 2.5f;

	/** How far in the future the agent reacts to environment collisions. */
	UPROPERTY(EditAnywhere, Category = "Predictive Avoidance", meta = (ClampMin = "0.1", ForceUnits="s"))
	float EnvironmentPredictiveAvoidanceTime = 1.f;

	/** Agent radius scale for anticipatory avoidance. Making the scale smaller makes the agent more eager to squeeze through other agents. Range: 0.5..1 */
	UPROPERTY(EditAnywhere, Category = "Predictive Avoidance", meta = (ClampMin = "0", ForceUnits="x"))
	float PredictiveAvoidanceRadiusScale = 0.65f;
	
	/** Predictive avoidance force effect distance. The avoidance force is applied at the point in future where the agents are closest. The actual observed separation distance will be smaller. Range: 0..200 */
	UPROPERTY(EditAnywhere, Category = "Predictive Avoidance", meta = (ClampMin = "0", ForceUnits="cm"))
	float PredictiveAvoidanceDistance = 75.f;

	/** Predictive environment avoidance force effect distance. The avoidance force is applied at the point in future where the agents are closest. The actual observed separation distance will be smaller. Range: 0..100 */
	UPROPERTY(EditAnywhere, Category = "Predictive Avoidance", meta = (ClampMin = "0", ForceUnits="cm"))
	float EnvironmentPredictiveAvoidanceDistance = 30.f;
	
	/** Predictive avoidance force stiffness between agents and obstacles. Range: 400..1000 N/cm */
	UPROPERTY(EditAnywhere, Category = "Predictive Avoidance", meta = (ClampMin = "0"))
	float ObstaclePredictiveAvoidanceStiffness = 700.f;

	/** Predictive avoidance force stiffness between agents and walls. Range: 400..1000 N/cm */
	UPROPERTY(EditAnywhere, Category = "Predictive Avoidance", meta = (ClampMin = "0"))
	float EnvironmentPredictiveAvoidanceStiffness = 200.f;
};

USTRUCT()
struct FMassStandingAvoidanceParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	FMassStandingAvoidanceParameters GetValidated() const
	{
		FMassStandingAvoidanceParameters Copy = *this;
		
		Copy.GhostSteeringReactionTime = FMath::Max(Copy.GhostSteeringReactionTime, KINDA_SMALL_NUMBER);

		return Copy;
	}

	/** The distance at which neighbour agents are detected when updating the ghost. */
	UPROPERTY(EditAnywhere, Category = "General", meta = (ClampMin = "0.0", ForceUnits="cm"))
	float GhostObstacleDetectionDistance = 300.f;

	/** How far the ghost can deviate from the target location. */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="cm"))
	float GhostToTargetMaxDeviation = 80.0f;

	/**  */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="s"))
	float GhostSteeringReactionTime = 2.0f;

	/** The steering will slow down when the ghost is closer than this distance to the target. Range: 5..50 */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="cm"))
	float GhostStandSlowdownRadius = 15.0f;

	/** Mas speed the ghost can move. */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="cm/s"))
	float GhostMaxSpeed = 250.0f;

	/** Max acceleration of the ghost. Making this larger than the agent speed will make the ghost react quickly.  */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="cm/s"))
	float GhostMaxAcceleration = 300.0f;

	/** How quickly the ghost speed goes to zero. The smaller the value, the more the movement is dampened. */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="s"))
	float GhostVelocityDampingTime = 0.4f;

	/** Agent radius scale for separation. Making it smaller makes the separation softer. Range: 0.8..1 */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="x"))
	float GhostSeparationRadiusScale = 0.8f;
	
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="cm"))
	float GhostSeparationDistance = 20.0f;
	
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="N/cm"))
	float GhostSeparationStiffness = 200.0f;

	/** How much avoidance is scaled for moving obstacles. Range: 1..5. */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="x"))
	float MovingObstacleAvoidanceScale = 3.0f;

	/** How much the ghost avoidance is tuned down when the moving obstacle is moving away from the ghost. Range: 0..1 */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="x"))
	float MovingObstacleDirectionalScale = 0.1f;

	/** How much extra space is preserved in front of moving obstacles (relative to their size). Range: 1..5 */
	UPROPERTY(EditAnywhere, Category = "Ghost", meta = (ClampMin = "0", ForceUnits="x"))
	float MovingObstaclePersonalSpaceScale = 3.0f;
};


/** Edge with normal */
struct FNavigationAvoidanceEdge
{
	FNavigationAvoidanceEdge(const FVector InStart, const FVector InEnd)
	{
		Start = InStart;
		End = InEnd;
		LeftDir = FVector::CrossProduct((End - Start), FVector::UpVector).GetSafeNormal();
	}
	FVector Start = FVector::ZeroVector;
	FVector End = FVector::ZeroVector;
	FVector LeftDir = FVector::ZeroVector;
};

USTRUCT()
struct FMassNavigationEdgesFragment : public FMassFragment
{
	GENERATED_BODY()

	static constexpr int MaxEdgesCount = 16;
	TArray<FNavigationAvoidanceEdge, TFixedAllocator<MaxEdgesCount>> AvoidanceEdges;

	/* Set to true if the edges already account for the agent radius. */
	bool bExtrudedEdges = false;
};

template<>
struct TMassFragmentTraits<FMassNavigationEdgesFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};

USTRUCT()
struct FMassAvoidanceEntitiesToIgnoreFragment : public FMassFragment
{
	GENERATED_BODY()

	static constexpr int EntityCount = 4;
	TArray<FMassEntityHandle, TFixedAllocator<EntityCount>> EntitiesToIgnore;
};

template<>
struct TMassFragmentTraits<FMassAvoidanceEntitiesToIgnoreFragment> final
{
	enum
	{
		AuthorAcceptsItsNotTriviallyCopyable = true
	};
};