// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassEntityQuery.h"
#include "MassAvoidanceProcessors.generated.h"

#define UE_API MASSNAVIGATION_API

MASSNAVIGATION_API DECLARE_LOG_CATEGORY_EXTERN(LogAvoidance, Warning, All);
MASSNAVIGATION_API DECLARE_LOG_CATEGORY_EXTERN(LogAvoidanceVelocities, Warning, All);
MASSNAVIGATION_API DECLARE_LOG_CATEGORY_EXTERN(LogAvoidanceAgents, Warning, All);
MASSNAVIGATION_API DECLARE_LOG_CATEGORY_EXTERN(LogAvoidanceObstacles, Warning, All);

class UMassNavigationSubsystem;

/** Experimental: move using cumulative forces to avoid close agents */
UCLASS(MinimalAPI)
class UMassMovingAvoidanceProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassMovingAvoidanceProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};

/** Avoidance while standing. */
UCLASS(MinimalAPI)
class UMassStandingAvoidanceProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassStandingAvoidanceProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};

#undef UE_API
