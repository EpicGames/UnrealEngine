// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassAvoidanceFragments.h"
#include "MassAvoidanceTrait.generated.h"

#define UE_API MASSNAVIGATION_API

UCLASS(MinimalAPI, meta = (DisplayName = "Avoidance"))
class UMassObstacleAvoidanceTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(EditAnywhere, Category="")
	FMassMovingAvoidanceParameters MovingParameters;
	
	UPROPERTY(EditAnywhere, Category="")
	FMassStandingAvoidanceParameters StandingParameters;
};

#undef UE_API
