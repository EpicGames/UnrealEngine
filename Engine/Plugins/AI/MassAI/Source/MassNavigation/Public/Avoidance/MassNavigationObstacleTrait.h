// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassNavigationObstacleTrait.generated.h"

#define UE_API MASSNAVIGATION_API

UCLASS(MinimalAPI, meta = (DisplayName = "Navigation Obstacle"))
class UMassNavigationObstacleTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

};

#undef UE_API
