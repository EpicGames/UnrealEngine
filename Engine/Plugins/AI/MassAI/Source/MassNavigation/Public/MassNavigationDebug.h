// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"

#if WITH_MASSGAMEPLAY_DEBUG
#include "MassDebugLogging.h"
#include "Mass/EntityHandle.h"

struct FMassExecutionContext;

namespace UE::MassNavigation::Debug
{
	/** If true, will log all debug draw events regardless of debug entity selection if the visual log recorder is activated */
	MASSNAVIGATION_API bool ShouldLogEverythingWhenRecording();
	
	struct FDebugContext
	{
		FDebugContext(const FMassExecutionContext& Context, const UObject* InFallbackLogOwner, const FLogCategoryBase& InCategory, const UWorld* InWorld, const FMassEntityHandle InEntity, int32 InEntityIndex)
			: LogContext(Context, ShouldLogEverythingWhenRecording())
			, FallbackLogOwner(InFallbackLogOwner)
			, Category(InCategory)
			, World(InWorld)
			, Entity(InEntity)
			, EntityIndex(InEntityIndex)
		{}

		FDebugContext(const FMassExecutionContext& Context, const UObject* InFallbackLogOwner, const FLogCategoryBase& InCategory, const FMassEntityHandle InEntity, int32 InEntityIndex)
			: LogContext(Context, ShouldLogEverythingWhenRecording())
			, FallbackLogOwner(InFallbackLogOwner)
			, Category(InCategory)
			, World(Mass::Debug::GetDebugDrawWorld(Context))
			, Entity(InEntity)
			, EntityIndex(InEntityIndex)
		{}

		const UObject* GetLogOwner() const
		{
			return LogContext.GetLogOwner(EntityIndex, FallbackLogOwner);
		}
		
		bool ShouldLogEntity(FColor* OutEntityColor = nullptr) const
		{
			return LogContext.ShouldLogEntity(EntityIndex, OutEntityColor);
		}

		const Mass::Debug::FLoggingContext LogContext;
		
		/** If entity has no debug owner set, use this instead. Usually the mass processor */
		const UObject* FallbackLogOwner;
		const FLogCategoryBase& Category;
		const UWorld* World;
		const FMassEntityHandle Entity;
		
		/** EntityIndex is index in current chunk */
		const int32 EntityIndex;
	};
	
	MASSNAVIGATION_API bool UseDrawDebugHelper();
	MASSNAVIGATION_API bool DebugIsSelected(const FMassEntityHandle Entity);

	MASSNAVIGATION_API void DebugDrawLine(const FDebugContext& Context, const FVector& Start, const FVector& End, const FColor& Color,
		const float Thickness = 0.f, const bool bPersistent = false, const FString& Text = FString());

	MASSNAVIGATION_API void DebugDrawArrow(const FDebugContext& Context, const FVector& Start, const FVector& End, const FColor& Color,
		const float HeadSize = 8.f, const float Thickness = 1.5f, const FString& Text = FString());

	MASSNAVIGATION_API void DebugDrawSphere(const FDebugContext& Context, const FVector& Center, const FVector::FReal InRadius, const FColor& Color,
		const FString& Text = FString());

	MASSNAVIGATION_API void DebugDrawBox(const FDebugContext& Context, const FBox& Box, const FColor& Color, const FString& Text = FString());

	MASSNAVIGATION_API void DebugDrawCylinder(const FDebugContext& Context, const FVector& Bottom, const FVector& Top, const FVector::FReal InRadius,
		const FColor& Color, const FString& Text = FString());

	MASSNAVIGATION_API void DebugDrawCircle(const FDebugContext& Context, const FVector& Bottom, const FVector::FReal InRadius, const FColor& Color,
		const FString& Text = FString());

	inline FColor MixColors(const FColor ColorA, const FColor ColorB)
	{
		const int32 R = ((int32)ColorA.R + (int32)ColorB.R) / 2;
		const int32 G = ((int32)ColorA.G + (int32)ColorB.G) / 2;
		const int32 B = ((int32)ColorA.B + (int32)ColorB.B) / 2;
		const int32 A = ((int32)ColorA.A + (int32)ColorB.A) / 2;
		return FColor((uint8)R, (uint8)G, (uint8)B, (uint8)A);
	}
} // UE::MassNavigation::Debug

#endif // WITH_MASSGAMEPLAY_DEBUG
