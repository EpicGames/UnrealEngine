// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "AI/Navigation/NavigationElement.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_8
#include "MassCommonTypes.h"
#include "MassEntityTypes.h"
#include "MassNavigationSubsystem.h"
#include "MassNavigationTypes.h"
#include "MassNavigationFragments.generated.h"

class UWorld;

/** Move target. */
USTRUCT()
struct FMassMoveTargetFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassMoveTargetFragment() : bNetDirty(false), bOffBoundaries(false), bSteeringFallingBehind(false) {}

	/** To setup current action from the authoritative world */
	MASSNAVIGATION_API void CreateNewAction(const EMassMovementAction InAction, const UWorld& InWorld);

	/** To setup current action from replicated data */
	MASSNAVIGATION_API void CreateReplicatedAction(const EMassMovementAction InAction, const uint16 InActionID, const double InWorldStartTime, const double InServerStartTime);

	void MarkNetDirty() { bNetDirty = true; }
	bool GetNetDirty() const { return bNetDirty; }
	void ResetNetDirty() { bNetDirty = false; }

public:
	MASSNAVIGATION_API FString ToString() const;

	EMassMovementAction GetPreviousAction() const { return PreviousAction; }
	EMassMovementAction GetCurrentAction() const { return CurrentAction; }
	double GetCurrentActionStartTime() const { return CurrentActionWorldStartTime; }
	double GetCurrentActionServerStartTime() const { return CurrentActionServerStartTime; }
	uint16 GetCurrentActionID() const { return CurrentActionID; }

	static MASSNAVIGATION_API const float UnsetDistance;

	/** Center of the move target. */
	FVector Center = FVector::ZeroVector;

	/** Forward direction of the movement target.  */
	FVector Forward = FVector::ZeroVector;

	/** Distance remaining to the movement goal. */
	float DistanceToGoal = 0.0f;

	/** Projected progress distance of the entity using the path. */
	float EntityDistanceToGoal = UnsetDistance;

	/** Allowed deviation around the movement target. */
	float SlackRadius = 0.0f;

private:
	/** World time in seconds when the action started. */
	double CurrentActionWorldStartTime = 0.0;

	/** Server time in seconds when the action started. */
	double CurrentActionServerStartTime = 0.0;

	/** Number incremented each time new action (i.e move, stand, animation) is started. */
	uint16 CurrentActionID = 0;

public:
	/** Requested movement speed. */
	FMassInt16Real DesiredSpeed = FMassInt16Real(0.0f);

	/** Intended movement action at the target. */
	EMassMovementAction IntentAtGoal = EMassMovementAction::Move;

private:
	/** Current movement action. */
	EMassMovementAction CurrentAction = EMassMovementAction::Move;

	/** Previous movement action. */
	EMassMovementAction PreviousAction = EMassMovementAction::Move;

	uint8 bNetDirty : 1;
public:
	/** True if the movement target is assumed to be outside navigation boundaries. */
	uint8 bOffBoundaries : 1;

	/** True if the movement target is assumed to be outside navigation boundaries. */
	uint8 bSteeringFallingBehind : 1;
};

/** Ghost location used for standing navigation. */
USTRUCT()
struct FMassGhostLocationFragment : public FMassFragment
{
	GENERATED_BODY()

	bool IsValid(const uint16 CurrentActionID) const
	{
		return LastSeenActionID == CurrentActionID;
	}

	/** The action ID the ghost was initialized for */
	uint16 LastSeenActionID = 0;

	/** Location of the ghost */
	FVector Location = FVector::ZeroVector;
	
	/** Velocity of the ghost */
	FVector Velocity = FVector::ZeroVector;
};

/** Cell location for dynamic obstacles */
USTRUCT()
struct FMassNavigationObstacleGridCellLocationFragment : public FMassFragment
{
	GENERATED_BODY()
	FNavigationObstacleHashGrid2D::FCellLocation CellLoc;
};


enum class EMassColliderType : uint8
{
	Circle,
	Pill,
};

struct FMassCircleCollider
{
	FMassCircleCollider() = default;
	FMassCircleCollider(const float Radius) : Radius(Radius) {}
	float Radius = 0.f;
};

struct FMassPillCollider
{
	FMassPillCollider() = default;
	FMassPillCollider(const float Radius, const float HalfLength) : Radius(Radius), HalfLength(HalfLength) {}
	float Radius = 0.f;
	float HalfLength = 0.f;
};

/** Fragment holding data for avoidance colliders */
USTRUCT()
struct FMassAvoidanceColliderFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassAvoidanceColliderFragment()
	{
		Type = EMassColliderType::Circle;
		Data[0] = 0.f;
		Data[1] = 0.f;
	}

	FMassAvoidanceColliderFragment(const FMassCircleCollider& Circle)
	{
		Type = EMassColliderType::Circle;
		Data[0] = Circle.Radius;
		Data[1] = 0.f;
	}

	FMassAvoidanceColliderFragment(const FMassPillCollider& Pill)
	{
		Type = EMassColliderType::Pill;
		Data[0] = Pill.Radius;
		Data[1] = Pill.HalfLength;
	}
	
	FMassCircleCollider GetCircleCollider() const
	{
		check(Type == EMassColliderType::Circle);
		return FMassCircleCollider(Data[0]);
	}

	FMassPillCollider GetPillCollider() const
	{
		check(Type == EMassColliderType::Pill);
		return FMassPillCollider(Data[0], Data[1]);
	}

	float Data[2];
	EMassColliderType Type;
};


/** Tag to tell if the entity is in the navigation obstacle grid */
USTRUCT()
struct FMassInNavigationObstacleGridTag : public FMassTag
{
	GENERATED_BODY()
};

/** Tag used to prevent avoidance from generating backward movements */
USTRUCT()
struct FMassAvoidancePreventBackwardMovementTag : public FMassTag
{
	GENERATED_BODY()
};

UE_DEPRECATED(5.8, "Renamed to FMassNavigationRelevantParameters.") typedef struct FNavigationRelevantParameters FMassNavigationRelevantParameters;
UE_DEPRECATED(5.8, "Renamed to FMassNavigationRelevantFragment") typedef struct FNavigationRelevantFragment FMassNavigationRelevantFragment;
