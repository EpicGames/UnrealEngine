// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "MassObserverProcessor.h"
#include "MassNavigationProcessors.generated.h"

#define UE_API MASSNAVIGATION_API

class UMassNavigationSubsystem;

/**
 * Updates Off-LOD entities position to move targets position.
 */
UCLASS(MinimalAPI)
class UMassOffLODNavigationProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassOffLODNavigationProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery_Conditional;
};

/**
 * Updates entities height to move targets position smoothly.
 * Does not update Off-LOD entities.
 */
UCLASS(MinimalAPI)
class UMassNavigationSmoothHeightProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassNavigationSmoothHeightProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};

/**
 * Initializes the move target's location to the agents initial position.
 */
UCLASS(MinimalAPI)
class UMassMoveTargetFragmentInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()
public:
	UE_API UMassMoveTargetFragmentInitializer();
	
protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery InitializerQuery;
};

/** Processor to update obstacle grid */
UCLASS(MinimalAPI)
class UMassNavigationObstacleGridProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassNavigationObstacleGridProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery AddToGridEntityQuery;
	FMassEntityQuery UpdateGridEntityQuery;
	FMassEntityQuery RemoveFromGridEntityQuery;
};

/** Deinitializer processor to remove avoidance obstacles from the avoidance obstacle grid */
UCLASS(MinimalAPI)
class UMassNavigationObstacleRemoverProcessor : public UMassObserverProcessor
{
	GENERATED_BODY()

	UMassNavigationObstacleRemoverProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

#undef UE_API
