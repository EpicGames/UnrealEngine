// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Mass/EntityHandle.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "HierarchicalHashGrid2D.h"
#include "MassSubsystemBase.h"
#include "Mass/ExternalSubsystemTraits.h"
#include "MassNavigationSubsystem.generated.h"

UENUM()
enum class EMassNavigationObstacleFlags : uint8
{
	None			= 0,
	HasColliderData = 1 << 0,
};
ENUM_CLASS_FLAGS(EMassNavigationObstacleFlags)

struct FMassNavigationObstacleItem
{
	bool operator==(const FMassNavigationObstacleItem& Other) const
	{
		return Entity == Other.Entity;
	}

	FMassEntityHandle Entity;
	EMassNavigationObstacleFlags ItemFlags = EMassNavigationObstacleFlags::None;
};

typedef THierarchicalHashGrid2D<2, 4, FMassNavigationObstacleItem> FNavigationObstacleHashGrid2D;	// 2 levels of hierarchy, 4 ratio between levels

UCLASS(MinimalAPI)
class UMassNavigationSubsystem : public UMassSubsystemBase
{
	GENERATED_BODY()

public:
	MASSNAVIGATION_API UMassNavigationSubsystem();

	const FNavigationObstacleHashGrid2D& GetObstacleGrid() const { return AvoidanceObstacleGrid; }
	FNavigationObstacleHashGrid2D& GetObstacleGridMutable() { return AvoidanceObstacleGrid; }

protected:
	// USubsystem BEGIN
	MASSNAVIGATION_API virtual void Initialize(FSubsystemCollectionBase& Collection) override;
	// USubsystem END

	FNavigationObstacleHashGrid2D AvoidanceObstacleGrid;
};

template<>
struct TMassExternalSubsystemTraits<UMassNavigationSubsystem> final
{
	enum
	{
		GameThreadOnly = false,
		ThreadSafeWrite = false
	};
};
