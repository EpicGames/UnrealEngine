// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "MassCommonTypes.h"
#include "RandomSequence.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassNavigationTypes.generated.h"

MASSNAVIGATION_API DECLARE_LOG_CATEGORY_EXTERN(LogMassNavigation, Warning, All);
MASSNAVIGATION_API DECLARE_LOG_CATEGORY_EXTERN(LogMassNavigationObstacle, Warning, All);

//@ todo remove optimization hack once we find a better way to filter out signals from LOD/listener on signals, as for now we only need this signal for look at in high and med LOD
#define HACK_DISABLE_PATH_CHANGED_ON_LOWER_LOD 1

namespace UE::Mass::Signals
{
	const FName FollowPointPathStart = FName(TEXT("FollowPointPathStart"));
	const FName FollowPointPathDone = FName(TEXT("FollowPointPathDone"));
	const FName CurrentLaneChanged = FName(TEXT("CurrentLaneChanged"));
}

UENUM()
enum class EMassMovementAction : uint8
{
	Stand,		// Stop and stand.
	Move,		// Move or keep on moving.
	Animate,	// Animation has control over the transform
};

/** Target location used by tasks */
USTRUCT()
struct FMassTargetLocation
{
	GENERATED_BODY()

	void Reset()
	{
		EndOfPathPosition.Reset();
		EndOfPathRotation.Reset();
		EndOfPathIntent = EMassMovementAction::Move;
	}

	/** Optional end of path location. */
	UPROPERTY(VisibleAnywhere, Category = Navigation, transient)
	TOptional<FVector> EndOfPathPosition;

	/** Optional end of path rotation. */
	UPROPERTY(VisibleAnywhere, Category = Navigation, transient)
	TOptional<FQuat> EndOfPathRotation;

	/** Movement intent at the end of the path */
	UPROPERTY(VisibleAnywhere, Category = Navigation, transient)
	EMassMovementAction EndOfPathIntent = EMassMovementAction::Move;
};
