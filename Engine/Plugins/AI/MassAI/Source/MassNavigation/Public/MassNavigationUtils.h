// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Quat.h"
#include "Math/UnrealMathUtility.h"

namespace UE::MassNavigation
{
	// Calculates yaw angle from direction vector.
	inline FQuat::FReal GetYawFromQuat(const FQuat Rotation)
	{
		const FQuat::FReal YawY = 2. * (Rotation.W * Rotation.Z + Rotation.X * Rotation.Y);
		const FQuat::FReal YawX = (1. - 2. * (FMath::Square(Rotation.Y) + FMath::Square(Rotation.Z)));
		return FMath::Atan2(YawY, YawX);
	}

	inline FVector::FReal GetYawFromDirection(const FVector Direction)
	{
		return FMath::Atan2(Direction.Y, Direction.X);
	}

	// Wraps and angle to range -PI..PI. Angle in radians.
	inline FVector::FReal WrapAngle(const FVector::FReal Angle)
	{
		FVector::FReal WrappedAngle = FMath::Fmod(Angle, UE_DOUBLE_PI*2.);
		WrappedAngle = (WrappedAngle > UE_DOUBLE_PI) ? WrappedAngle - UE_DOUBLE_PI * 2. : WrappedAngle;
		WrappedAngle = (WrappedAngle < -UE_DOUBLE_PI) ? WrappedAngle + UE_DOUBLE_PI * 2. : WrappedAngle;
		return WrappedAngle;
	}

	// Linearly interpolates between two angles (in Radians).
	inline FVector::FReal LerpAngle(const FVector::FReal AngleA, const FVector::FReal AngleB, const FVector::FReal T)
	{
		const FVector::FReal DeltaAngle = WrapAngle(AngleB - AngleA);
		return AngleA + DeltaAngle * T;
	}

	// Exponential smooth from current angle to target angle. Angles in radians.
	inline FVector::FReal ExponentialSmoothingAngle(const FVector::FReal Angle, const FVector::FReal TargetAngle, const FVector::FReal DeltaTime, const FVector::FReal SmoothingTime)
	{
		// Note: based on FMath::ExponentialSmoothingApprox().
		if (SmoothingTime < KINDA_SMALL_NUMBER)
		{
			return TargetAngle;
		}
		const FVector::FReal A = DeltaTime / SmoothingTime;
		const FVector::FReal Exp = FMath::InvExpApprox(A);
		return TargetAngle + WrapAngle(Angle - TargetAngle) * Exp;
	}

	// Clamps vectors magnitude to Mag.
	inline FVector ClampVector(const FVector Vec, const FVector::FReal Mag)
	{
		const FVector::FReal Len = Vec.SizeSquared();
		if (Len > FMath::Square(Mag)) {
			return Vec * Mag / FMath::Sqrt(Len);
		}
		return Vec;
	}

	// Projects a point to segment and returns the time interpolation value.
	inline FVector::FReal ProjectPtSeg(const FVector2D Point, const FVector2D Start, const FVector2D End)
	{
		const FVector2D Seg = End - Start;
		const FVector2D Dir = Point - Start;
		const FVector::FReal SegSizeSquared = Seg.SizeSquared();
		const FVector::FReal SegDirDot = FVector2D::DotProduct(Seg, Dir);

		if (SegDirDot < 0.)
		{
			return 0.;
		}

		if (SegDirDot > SegSizeSquared)
		{
			return 1.;
		}

		return SegSizeSquared > 0. ? (SegDirDot / SegSizeSquared) : 0.;
	}

	// Returns the SmoothStep curve for X in range [0..1]. 
	inline float Smooth(const float X)
	{
		return X * X * (3.f - 2.f * X);
	}

	// Returns the SmoothStep curve for X in range [0..1]. 
	inline double Smooth(const double X)
	{
		return X * X * (3. - 2. * X);
	}

	// Returns left direction from forward and up directions.
	inline FVector GetLeftDirection(const FVector Forward, const FVector Up)
	{
		return FVector::CrossProduct(Forward, Up).GetSafeNormal();
	}

	// Computes miter normal in XY plane from two neighbour edge normals.
	inline FVector ComputeMiterNormal(const FVector NormalA, const FVector NormalB)
	{
		FVector Mid = 0.5 * (NormalA + NormalB);
		const FVector::FReal MidSquared = FVector::DotProduct(Mid, Mid);
		if (MidSquared > KINDA_SMALL_NUMBER)
		{
			const FVector::FReal Scale = FMath::Min(1. / MidSquared, 20.);
			Mid *= Scale;
		}
		return Mid;
	}
} // UE::MassNavigation
