// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassCommonTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "MassSmoothOrientationFragments.generated.h"


USTRUCT()
struct FMassSmoothOrientationWeights : public FMassSharedFragment
{
	GENERATED_BODY()

	FMassSmoothOrientationWeights() = default;

	FMassSmoothOrientationWeights(const float InMoveTargetWeight, const float InVelocityWeight)
		: MoveTargetWeight(InMoveTargetWeight)
		, VelocityWeight(InVelocityWeight)
	{
	}
	
	UPROPERTY(EditAnywhere, Category = "Orientation", meta = (ClampMin = "0.0", ClampMax="1.0"))
	float MoveTargetWeight = 0.0f;

	UPROPERTY(EditAnywhere, Category = "Orientation", meta = (ClampMin = "0.0", ClampMax="1.0"))
	float VelocityWeight = 0.0f;
};

USTRUCT()
struct FMassSmoothOrientationParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	/** The time it takes the orientation to catchup to the requested orientation. */
	UPROPERTY(EditAnywhere, Category = "Orientation", meta = (ClampMin = "0.0", ForceUnits="s"))
	float EndOfPathDuration = 1.0f;
	
	/** The time it takes the orientation to catchup to the requested orientation. */
	UPROPERTY(EditAnywhere, Category = "Orientation", meta = (ClampMin = "0.0", ForceUnits="s"))
	float OrientationSmoothingTime = 0.3f;

	/* Orientation blending weights while moving. */	
	UPROPERTY(EditAnywhere, Category = "Orientation")
	FMassSmoothOrientationWeights Moving = FMassSmoothOrientationWeights(/*MoveTarget*/0.4f, /*Velocity*/0.6f);

	/* Orientation blending weights while standing. */	
	UPROPERTY(EditAnywhere, Category = "Orientation")
	FMassSmoothOrientationWeights Standing = FMassSmoothOrientationWeights(/*MoveTarget*/0.95f, /*Velocity*/0.05f);
};
