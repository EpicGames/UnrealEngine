// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#include "MassSmoothOrientationProcessors.generated.h"

#define UE_API MASSNAVIGATION_API

/**
 * Updates agent's orientation based on current movement.
 */
UCLASS(MinimalAPI)
class UMassSmoothOrientationProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassSmoothOrientationProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery HighResEntityQuery;
	FMassEntityQuery LowResEntityQuery_Conditional;
};

#undef UE_API
