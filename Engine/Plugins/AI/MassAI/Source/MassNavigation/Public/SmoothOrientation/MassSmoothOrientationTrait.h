// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassSmoothOrientationFragments.h"
#include "MassSmoothOrientationTrait.generated.h"

#define UE_API MASSNAVIGATION_API

UCLASS(MinimalAPI, meta = (DisplayName = "Smooth Orientation"))
class UMassSmoothOrientationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(EditAnywhere, Category="")
	FMassSmoothOrientationParameters Orientation;
};

#undef UE_API
