// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "MassMovementFragments.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#include "MassSteeringFragments.generated.h"

class UWorld;

/** Steering fragment. */
USTRUCT()
struct FMassSteeringFragment : public FMassFragment
{
	GENERATED_BODY()

	void Reset()
	{
		DesiredVelocity = FVector::ZeroVector;
	}

	/** Cached desired velocity from steering. Note: not used for moving the entity. */
	FVector DesiredVelocity = FVector::ZeroVector;
};

/** Standing steering. */
USTRUCT()
struct FMassStandingSteeringFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Selected steer target based on ghost, updates periodically. */
	FVector TargetLocation = FVector::ZeroVector;

	/** Used during target update to see when the target movement stops */
	float TrackedTargetSpeed = 0.0f;

	/** Cooldown between target updates */
	float TargetSelectionCooldown = 0.0f;

	/** True if the target is being updated */
	bool bIsUpdatingTarget = false;

	/** True if we just entered from move action */
	bool bEnteredFromMoveAction = false;
};

/** Tag used to prevent steering from generating a slowdown force. */
USTRUCT()
struct FMassSteerToMoveTargetPreventSlowdownTag : public FMassTag
{
	GENERATED_BODY()
};

/** Steering related movement parameters. */
USTRUCT()
struct FMassMovingSteeringParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	/** Steering reaction time in seconds. */
	UPROPERTY(config, EditAnywhere, Category = "Moving", meta = (ClampMin = "0.05", ForceUnits="s"))
	float ReactionTime = 0.3f;

	/** How much we look ahead when steering. Affects how steeply we steer towards the goal and when to start to slow down at the end of the path. */
	UPROPERTY(EditAnywhere, Category = "Moving", meta = (ClampMin = "0", ForceUnits="s"))
	float LookAheadTime = 1.0f;

	/** When using FMassSteerToMoveTargetPreventSlowdownTag, maximum distance at which slowdown reduction is applied. */
	UPROPERTY(EditAnywhere, Category = "Moving", meta = (ClampMin = "0", ForceUnits="s"))
	float SteeringPreventSlowdownAttenuationDistance = 50.f;
	
	/** Allow directional and catchup speed variance. */
	UPROPERTY(EditAnywhere, Category = "Moving")
	bool bAllowSpeedVariance = true;
};

USTRUCT()
struct FMassStandingSteeringParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	/** How much the ghost should deviate from the target before updating the target. */
	UPROPERTY(EditAnywhere, Category = "Standing", meta = (ClampMin = "0.05", ForceUnits="cm"))
	float TargetMoveThreshold = 15.0f;
	
	UPROPERTY(EditAnywhere, Category = "Standing")
	float TargetMoveThresholdVariance = 0.1f;

	/** If the velocity is below this threshold, it is clamped to 0. This allows to prevent jittery movement when trying to be stationary. */
	UPROPERTY(EditAnywhere, Category = "Movement", meta = (ClampMin = "0.0", ForceUnits="cm/s"))
	float LowSpeedThreshold = 3.0f;

	/** How much the max speed can drop before we stop tracking it. */
	UPROPERTY(EditAnywhere, Category = "Standing", meta = (ClampMin = "0.05", ForceUnits="x"))
	float TargetSpeedHysteresisScale = 0.85f;

	/** Time between updates, varied randomly. */
	UPROPERTY(EditAnywhere, Category = "Standing", meta = (ClampMin = "0.05", ForceUnits="s"))
	float TargetSelectionCooldown = 1.5f;
	
	UPROPERTY(EditAnywhere, Category = "Standing")
	float TargetSelectionCooldownVariance = 0.5f;

	/** How much the target should deviate from the current location before updating the force on the agent. */
	UPROPERTY(EditAnywhere, Category = "Standing", meta = (ForceUnits="cm"))
	float DeadZoneRadius = 15.0f;
};
