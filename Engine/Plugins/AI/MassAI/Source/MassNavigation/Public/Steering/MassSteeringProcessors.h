// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityQuery.h"
#include "MassProcessor.h"
#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassObserverProcessor.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassSteeringProcessors.generated.h"

/** 
* Processor for updating steering towards MoveTarget.
*/
UCLASS(MinimalAPI)
class UMassSteerToMoveTargetProcessor : public UMassProcessor
{
	GENERATED_BODY()

protected:
	MASSNAVIGATION_API UMassSteerToMoveTargetProcessor();
	
	MASSNAVIGATION_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	MASSNAVIGATION_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};
