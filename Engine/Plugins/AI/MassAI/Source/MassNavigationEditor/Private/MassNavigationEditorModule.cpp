// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavigationEditorModule.h"
#include "Modules/ModuleManager.h"
#include "AssetToolsModule.h"
#include "PropertyEditorModule.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "MassNavigationEditor"

IMPLEMENT_MODULE(FMassNavigationEditorModule, MassNavigationEditor)

void FMassNavigationEditorModule::StartupModule()
{
	// Register the details customizer
	FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
}

void FMassNavigationEditorModule::ShutdownModule()
{
	UToolMenus::UnRegisterStartupCallback(this);
	UToolMenus::UnregisterOwner(this);

	if (FModuleManager::Get().IsModuleLoaded(TEXT("PropertyEditor")))
	{
		FPropertyEditorModule& PropertyModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>(TEXT("PropertyEditor"));
	}
}

#undef LOCTEXT_NAMESPACE
