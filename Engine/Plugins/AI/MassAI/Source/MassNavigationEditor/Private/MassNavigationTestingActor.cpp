// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassNavigationTestingActor.h"
#include "UObject/ConstructorHelpers.h"
#include "DebugRenderSceneProxy.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphDelegates.h"
#include "ZoneGraphRenderingUtilities.h"
#include "ZoneGraphData.h"
#include "Engine/World.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassNavigationTestingActor)

#if UE_ENABLE_DEBUG_DRAWING

//-----------------------------------------------------------------------------
// FMassNavigationTestingSceneProxy
//-----------------------------------------------------------------------------
FMassNavigationTestingSceneProxy::FMassNavigationTestingSceneProxy(const UPrimitiveComponent& InComponent)
	: FDebugRenderSceneProxy(&InComponent)
{
	DrawType = WireMesh;
}

SIZE_T FMassNavigationTestingSceneProxy::GetTypeHash() const
{
	static size_t UniquePointer;
	return reinterpret_cast<size_t>(&UniquePointer);
}

FPrimitiveViewRelevance FMassNavigationTestingSceneProxy::GetViewRelevance(const FSceneView* View) const
{
	FPrimitiveViewRelevance Result;
	Result.bDrawRelevance = IsShown(View);
	Result.bDynamicRelevance = true;
	Result.bShadowRelevance = IsShadowCast(View);
	Result.bEditorPrimitiveRelevance = UseEditorCompositing(View);
	return Result;
}

uint32 FMassNavigationTestingSceneProxy::GetMemoryFootprint(void) const
{
	return sizeof(*this) + FDebugRenderSceneProxy::GetAllocatedSize();
}

#endif // UE_ENABLE_DEBUG_DRAWING

//-----------------------------------------------------------------------------
// UMassNavigationTestingComponent
//-----------------------------------------------------------------------------
UMassNavigationTestingComponent::UMassNavigationTestingComponent(const FObjectInitializer& ObjectInitialize)
	: Super(ObjectInitialize)
{
	SearchExtent = FVector(150.0f);
}

#if WITH_EDITOR
void UMassNavigationTestingComponent::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	UpdateTests();
}
#endif

void UMassNavigationTestingComponent::OnRegister()
{
	Super::OnRegister();
#if WITH_EDITOR
	// Force to update tests when ever the data changes.
	OnDataChangedHandle = UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.AddUObject(this, &UMassNavigationTestingComponent::OnZoneGraphDataBuildDone);
#endif
	OnDataAddedHandle = UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.AddUObject(this, &UMassNavigationTestingComponent::OnZoneGraphDataChanged);
	OnDataRemovedHandle = UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.AddUObject(this, &UMassNavigationTestingComponent::OnZoneGraphDataChanged);

	ZoneGraph = UWorld::GetSubsystem<UZoneGraphSubsystem>(GetWorld());

	UpdateTests();
}

void UMassNavigationTestingComponent::OnUnregister()
{
	Super::OnUnregister();
#if WITH_EDITOR
	UE::ZoneGraphDelegates::OnZoneGraphDataBuildDone.Remove(OnDataChangedHandle);
#endif
	UE::ZoneGraphDelegates::OnPostZoneGraphDataAdded.Remove(OnDataAddedHandle);
	UE::ZoneGraphDelegates::OnPreZoneGraphDataRemoved.Remove(OnDataRemovedHandle);
}

void UMassNavigationTestingComponent::OnZoneGraphDataChanged(const AZoneGraphData* ZoneGraphData)
{
	// Only consider valid graph from our world
	if (ZoneGraphData == nullptr || ZoneGraphData->GetWorld() != GetWorld())
	{
		return;
	}

	UpdateTests();
}

#if WITH_EDITOR
void UMassNavigationTestingComponent::OnZoneGraphDataBuildDone(const struct FZoneGraphBuildData& BuildData)
{
	UpdateTests();
}
#endif

FBoxSphereBounds UMassNavigationTestingComponent::CalcBounds(const FTransform& LocalToWorld) const
{
	const FVector ActorPosition = LocalToWorld.GetTranslation();
	return FBox(ActorPosition - SearchExtent, ActorPosition + SearchExtent);
}

void UMassNavigationTestingComponent::UpdateTests()
{
	if (!ZoneGraph)
	{
		return;
	}

	const FVector WorldPosition = GetOwner()->GetActorLocation();

	// Find nearest
	float DistanceSqr = 0.0f;
	if (PinnedLane.IsValid())
	{
		ZoneGraph->FindNearestLocationOnLane(LaneLocation.LaneHandle, FBox(WorldPosition - SearchExtent, WorldPosition + SearchExtent), LaneLocation, DistanceSqr);
	}
	else
	{
		ZoneGraph->FindNearestLane(FBox(WorldPosition - SearchExtent, WorldPosition + SearchExtent), QueryFilter, LaneLocation, DistanceSqr);
	}

	CachedLane.Reset();
	ShortPaths.Reset();

	GoalLaneLocation.Reset();

	if (LaneLocation.IsValid())
	{
		const FVector GoalWorldPosition = WorldPosition + GoalPosition;
		ZoneGraph->FindNearestLocationOnLane(LaneLocation.LaneHandle, FBox(GoalWorldPosition - SearchExtent, GoalWorldPosition + SearchExtent), GoalLaneLocation, DistanceSqr);

		if (const FZoneGraphStorage* ZoneStorage = ZoneGraph->GetZoneGraphStorage(LaneLocation.LaneHandle.DataHandle))
		{
			constexpr float InflateDistance = 200.0f;
			CachedLane.CacheLaneData(*ZoneStorage, LaneLocation.LaneHandle, LaneLocation.DistanceAlongLane, GoalLaneLocation.DistanceAlongLane, InflateDistance);

			// Path find has smaller buffer than cache, calculate up to 5 paths. Also allows to test that one path find can be continued to next one.
			FZoneGraphShortPathRequest PathRequest;
			PathRequest.bMoveReverse = GoalLaneLocation.DistanceAlongLane < LaneLocation.DistanceAlongLane;
			PathRequest.TargetDistance = GoalLaneLocation.DistanceAlongLane;
			if (bHasSpecificEndPoint)
			{
				PathRequest.EndOfPathPosition = GoalWorldPosition;
			}
			PathRequest.AnticipationDistance.Set(AnticipationDistance);

			PathRequest.StartPosition = WorldPosition;
			float StartDistanceAlongPath = LaneLocation.DistanceAlongLane;
			for (int32 Iter = 0; Iter < 5; Iter++)
			{
				FMassZoneGraphShortPathFragment& ShortPath = ShortPaths.AddDefaulted_GetRef();
				ShortPath.RequestPath(CachedLane, PathRequest, StartDistanceAlongPath, AgentRadius);

				if (ShortPath.NumPoints == 0 || !ShortPath.bPartialResult)
				{
					break;
				}

				// Restart from last lane end.
				PathRequest.StartPosition = ShortPath.Points[ShortPath.NumPoints - 1].Position;
				StartDistanceAlongPath = ShortPath.Points[ShortPath.NumPoints - 1].DistanceAlongLane.Get();
			}
		}
	}
	
	MarkRenderStateDirty();
}

void UMassNavigationTestingComponent::PinLane()
{
	if (LaneLocation.IsValid())
	{
		PinnedLane = LaneLocation.LaneHandle;
	}
	UpdateTests();
}

void UMassNavigationTestingComponent::ClearPinnedLane()
{
	PinnedLane.Reset();
	UpdateTests();
}

#if UE_ENABLE_DEBUG_DRAWING

FDebugRenderSceneProxy* UMassNavigationTestingComponent::CreateDebugSceneProxy()
{
	FMassNavigationTestingSceneProxy* DebugProxy = new FMassNavigationTestingSceneProxy(*this);

	const FTransform& ActorTransform = GetOwner()->GetActorTransform();
	const FVector Center = ActorTransform.GetLocation(); 
	const FVector Goal = Center + GoalPosition; 
	
	static constexpr float TickSize = 25.0f;

	DebugProxy->Boxes.Emplace(FBox(Center - SearchExtent, Center + SearchExtent), FColorList::SkyBlue);

	DebugProxy->Lines.Emplace(Center + FVector(-TickSize, 0, 0), Center + FVector(TickSize, 0, 0), FColorList::SkyBlue);
	DebugProxy->Lines.Emplace(Center + FVector(0, -TickSize, 0), Center + FVector(0, TickSize, 0), FColorList::SkyBlue);
	DebugProxy->Lines.Emplace(Center + FVector(0, 0, -TickSize), Center + FVector(0, 0, TickSize), FColorList::SkyBlue);

	DebugProxy->Lines.Emplace(Goal + FVector(-TickSize, 0, 0), Goal + FVector(TickSize, 0, 0), FColorList::YellowGreen);
	DebugProxy->Lines.Emplace(Goal + FVector(0, -TickSize, 0), Goal + FVector(0, TickSize, 0), FColorList::YellowGreen);
	DebugProxy->Lines.Emplace(Goal + FVector(0, 0, -TickSize), Goal + FVector(0, 0, TickSize), FColorList::YellowGreen);

	const FVector ZOffset(0,0,10);
	const FVector ZOffsetPath(0,0,20);
	
	if (LaneLocation.IsValid())
	{
		DebugProxy->Lines.Emplace(LaneLocation.Position - LaneLocation.Up * TickSize, LaneLocation.Position + LaneLocation.Up * TickSize * 3.0f, FColorList::SkyBlue);
		DebugProxy->Lines.Emplace(Center, LaneLocation.Position, FColorList::SkyBlue);
	}

	if (GoalLaneLocation.IsValid())
	{
		DebugProxy->Lines.Emplace(GoalLaneLocation.Position - GoalLaneLocation.Up * TickSize, GoalLaneLocation.Position + GoalLaneLocation.Up * TickSize * 3.0f, FColorList::YellowGreen);
		DebugProxy->Lines.Emplace(Goal, GoalLaneLocation.Position, FColorList::YellowGreen);
	}

	if (CachedLane.NumPoints > 1)
	{
		const float LeftSpace = CachedLane.LaneWidth.Get() * 0.5f + CachedLane.LaneLeftSpace.Get() - AgentRadius;
		const float RightSpace = CachedLane.LaneWidth.Get() * 0.5f + CachedLane.LaneRightSpace.Get() - AgentRadius;
		
		for (uint8 PointIndex = 0; PointIndex < CachedLane.NumPoints - 1; PointIndex++)
		{
			DebugProxy->Lines.Emplace(CachedLane.LanePoints[PointIndex] + ZOffset, CachedLane.LanePoints[PointIndex + 1] + ZOffset, FColorList::Grey, 4.0f);

			// Draw boundaries
			const FVector StartTangent = CachedLane.LaneTangentVectors[PointIndex].GetVector();
			const FVector StartLeftDir = FVector::CrossProduct(StartTangent, FVector::UpVector);
			const FVector StartLeftPos = CachedLane.LanePoints[PointIndex] + ZOffset + StartLeftDir * LeftSpace;
			const FVector StartRightPos = CachedLane.LanePoints[PointIndex] + ZOffset + StartLeftDir * -RightSpace;

			const FVector EndTangent = CachedLane.LaneTangentVectors[PointIndex + 1].GetVector();
			const FVector EndLeftDir = FVector::CrossProduct(EndTangent, FVector::UpVector);
			const FVector EndLeftPos = CachedLane.LanePoints[PointIndex + 1] + ZOffset + EndLeftDir * LeftSpace;
			const FVector EndRightPos = CachedLane.LanePoints[PointIndex + 1] + ZOffset + EndLeftDir * -RightSpace;

			DebugProxy->Lines.Emplace(StartLeftPos, EndLeftPos, FColorList::LightGrey, 1.0f);
			DebugProxy->Lines.Emplace(StartRightPos, EndRightPos, FColorList::LightGrey, 1.0f);

			if (PointIndex == 0)
			{
				DebugProxy->Lines.Emplace(StartLeftPos, StartRightPos, FColorList::LightGrey, 1.0f);
			}
			else if (PointIndex == (CachedLane.NumPoints - 1))
			{
				DebugProxy->Lines.Emplace(EndLeftPos, EndRightPos, FColorList::LightGrey, 1.0f);
			}
			
		}
	}

	for (int32 PathIndex = 0; PathIndex < ShortPaths.Num(); PathIndex++)
	{
		const FMassZoneGraphShortPathFragment& ShortPath = ShortPaths[PathIndex];
		const FColor PathColor = (PathIndex & 1) ? FColor::Orange : FColor::Emerald;
	
		if (ShortPath.NumPoints > 1)
		{
			for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints; PointIndex++)
			{
				const FVector Position = ShortPath.Points[PointIndex].Position + ZOffsetPath;
				const FVector LeftDir = FVector::CrossProduct(ShortPath.Points[PointIndex].Tangent.GetVector(), FVector::UpVector);

				DebugProxy->Lines.Emplace(Position - LeftDir * TickSize, Position + LeftDir * TickSize, PathColor, 2.0f);
				DebugProxy->Texts.Emplace(FString::Printf(TEXT("%d"), (int32)PointIndex), Position, PathColor);
			}

			for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints - 1; PointIndex++)
			{
				DebugProxy->Lines.Emplace(ShortPath.Points[PointIndex].Position + ZOffsetPath, ShortPath.Points[PointIndex + 1].Position + ZOffsetPath, PathColor, 4.0f);
			}
		}
	}

	return DebugProxy;
}
#endif // UE_ENABLE_DEBUG_DRAWING

//-----------------------------------------------------------------------------
// UMassNavigationTestingComponent
//-----------------------------------------------------------------------------
AMassNavigationTestingActor::AMassNavigationTestingActor(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DebugComp = CreateDefaultSubobject<UMassNavigationTestingComponent>(TEXT("DebugComp"));
	RootComponent = DebugComp;

	SetCanBeDamaged(false);
}

#if WITH_EDITOR
void AMassNavigationTestingActor::PostEditMove(bool bFinished)
{
	if (DebugComp)
	{
		DebugComp->UpdateTests();
	}
}
#endif

void AMassNavigationTestingActor::PinLane()
{
	if (DebugComp)
	{
		DebugComp->PinLane();
	}
}
	
void AMassNavigationTestingActor::ClearPinnedLane()
{
	if (DebugComp)
	{
		DebugComp->ClearPinnedLane();
	}
}
