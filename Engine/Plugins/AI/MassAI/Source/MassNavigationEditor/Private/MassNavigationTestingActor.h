// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "Debug/DebugDrawComponent.h"
#include "DebugRenderSceneProxy.h"
#include "ZoneGraphTypes.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassNavigationTestingActor.generated.h"

#define UE_API MASSNAVIGATIONEDITOR_API

class AZoneGraphData;
class UZoneGraphTestingComponent;
class UZoneGraphSubsystem;
class AMassNavigationTestingActor;

#if UE_ENABLE_DEBUG_DRAWING
class FMassNavigationTestingSceneProxy final : public FDebugRenderSceneProxy
{
public:
	UE_API FMassNavigationTestingSceneProxy(const UPrimitiveComponent& InComponent);
	
	UE_API virtual SIZE_T GetTypeHash() const override;
	UE_API virtual FPrimitiveViewRelevance GetViewRelevance(const FSceneView* View) const override;
	UE_API virtual uint32 GetMemoryFootprint(void) const override;
};
#endif // UE_ENABLE_DEBUG_DRAWING

/** Component for testing MassMovement functionality. */
UCLASS(MinimalAPI, ClassGroup = Debug)
class UMassNavigationTestingComponent : public UDebugDrawComponent
{
	GENERATED_BODY()
public:
	UE_API UMassNavigationTestingComponent(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif // WITH_EDITOR

	UE_API virtual void OnRegister() override;
	UE_API virtual void OnUnregister() override;

	UE_API virtual FBoxSphereBounds CalcBounds(const FTransform& LocalToWorld) const override;

#if UE_ENABLE_DEBUG_DRAWING
	UE_API virtual FDebugRenderSceneProxy* CreateDebugSceneProxy() override;
#endif

	UE_API void UpdateTests();
	UE_API void PinLane();
	UE_API void ClearPinnedLane();

protected:

#if WITH_EDITOR
	UE_API void OnZoneGraphDataBuildDone(const struct FZoneGraphBuildData& BuildData);
#endif
	UE_API void OnZoneGraphDataChanged(const AZoneGraphData* ZoneGraphData);

#if WITH_EDITOR
	FDelegateHandle OnDataChangedHandle;
#endif
	FDelegateHandle OnDataAddedHandle;
	FDelegateHandle OnDataRemovedHandle;

	UPROPERTY(Transient)
	TObjectPtr<UZoneGraphSubsystem> ZoneGraph;

	UPROPERTY(Transient)
	FZoneGraphLaneLocation LaneLocation;

	UPROPERTY(Transient)
	FZoneGraphLaneLocation GoalLaneLocation;

	UPROPERTY(EditAnywhere, Category = Default);
	FVector SearchExtent;

	UPROPERTY(EditAnywhere, Category = Default);
	float AnticipationDistance = 50.0f;

	UPROPERTY(EditAnywhere, Category = Default);
	float AgentRadius = 40.0f;

	UPROPERTY(EditAnywhere, Category = Default);
	bool bHasSpecificEndPoint = true;

	UPROPERTY(EditAnywhere, Category = Default);
	FZoneGraphTagFilter QueryFilter;

	UPROPERTY(EditAnywhere, Category = Default, meta = (MakeEditWidget=true))
	FVector GoalPosition;

	FZoneGraphLaneHandle PinnedLane;
	
	FMassZoneGraphCachedLaneFragment CachedLane;
	TArray<FMassZoneGraphShortPathFragment> ShortPaths;
};

/** Debug actor to visually test zone graph. */
UCLASS(MinimalAPI, hidecategories = (Actor, Input, Collision, Rendering, Replication, Partition, HLOD, Cooking))
class AMassNavigationTestingActor : public AActor
{
	GENERATED_BODY()
public:
	UE_API AMassNavigationTestingActor(const FObjectInitializer& ObjectInitializer = FObjectInitializer::Get());

#if WITH_EDITOR
	UE_API virtual void PostEditMove(bool bFinished) override;
#endif // WITH_EDITOR

	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Default")
	UE_API void PinLane();
	
	UFUNCTION(BlueprintCallable, CallInEditor, Category = "Default")
	UE_API void ClearPinnedLane();

protected:
	UPROPERTY(Category = Default, VisibleAnywhere, meta = (AllowPrivateAccess = "true"))
	TObjectPtr<UMassNavigationTestingComponent> DebugComp;
};

#undef UE_API
