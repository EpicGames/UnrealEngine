// Copyright Epic Games, Inc. All Rights Reserved.

#pragma  once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "CoreMinimal.h"
#include "AssetTypeCategories.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkitHost.h"
#include "Toolkits/AssetEditorToolkit.h"


class IMassNavigationEditor;

class FMassNavigationEditorModule : public IModuleInterface
{
public:
	// Begin IModuleInterface
	MASSNAVIGATIONEDITOR_API virtual void StartupModule() override;
	MASSNAVIGATIONEDITOR_API virtual void ShutdownModule() override;
	// End IModuleInterface

private:
};
