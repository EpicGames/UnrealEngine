// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassZoneGraphNavigationFragments.h"
#include "ZoneGraphTypes.h"
#include "ZoneGraphQuery.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphNavigationFragments)


namespace UE::MassNavigation::ZoneGraphPath
{
	struct FCachedLaneSegmentIterator
	{
		FCachedLaneSegmentIterator(const FMassZoneGraphCachedLaneFragment& InCachedLane, const float DistanceAlongPath, const bool bInMoveReverse)
			: CachedLane(InCachedLane)
			, SegmentInc(bInMoveReverse ? -1 : 1)
			, bMoveReverse(bInMoveReverse)
		{
			check(CachedLane.NumPoints >= 2);
			CurrentSegment = CachedLane.FindSegmentIndexAtDistance(DistanceAlongPath);
			LastSegment = bInMoveReverse ? 0 : ((int32)CachedLane.NumPoints - 2);
		}

		bool HasReachedDistance(const float Distance) const
		{
			if (CurrentSegment == LastSegment)
			{
				return true;
			}

			if (bMoveReverse)
			{
				const float SegStartDistance = CachedLane.LanePointProgressions[CurrentSegment].Get();
				if (Distance > SegStartDistance)
				{
					return true;
				}
			}
			else
			{
				const float SegEndDistance = CachedLane.LanePointProgressions[CurrentSegment + 1].Get();
				if (Distance < SegEndDistance)
				{
					return true;
				}
			}

			return false;
		}

		void Next()
		{
			if (CurrentSegment != LastSegment)
			{
				CurrentSegment += SegmentInc;
			}
		}

		const FMassZoneGraphCachedLaneFragment& CachedLane;
		int32 CurrentSegment = 0;
		int32 LastSegment = 0;
		int32 SegmentInc = 0;
		bool bMoveReverse = false;
	};

} // UE::MassMovement::ZoneGraphPath

void FMassZoneGraphCachedLaneFragment::CacheLaneData(const FZoneGraphStorage& ZoneGraphStorage, const FZoneGraphLaneHandle CurrentLaneHandle,
													 const float CurrentDistanceAlongLane, const float TargetDistanceAlongLane, const float InflateDistance)
{
	const FZoneLaneData& Lane = ZoneGraphStorage.Lanes[CurrentLaneHandle.Index];

	const float StartDistance = FMath::Min(CurrentDistanceAlongLane, TargetDistanceAlongLane);
	const float EndDistance = FMath::Max(CurrentDistanceAlongLane, TargetDistanceAlongLane);
	const float CurrentLaneLength = ZoneGraphStorage.LanePointProgressions[Lane.PointsEnd - 1];

	// If cached data contains the request part of the lane, early out.
	const float InflatedStartDistance = FMath::Max(0.0f, StartDistance - InflateDistance);
	const float InflatedEndDistance = FMath::Min(EndDistance + InflateDistance, CurrentLaneLength);
	if (LaneHandle == CurrentLaneHandle
		&& NumPoints > 0
		&& InflatedStartDistance >= LanePointProgressions[0].Get()
		&& InflatedEndDistance <= LanePointProgressions[NumPoints - 1].Get())
	{
		return;
	}

	Reset();
	CacheID++;

	LaneHandle = CurrentLaneHandle;
	LaneWidth = FMassInt16Real(Lane.Width);
	LaneLength = CurrentLaneLength;

	const int32 LaneNumPoints = Lane.PointsEnd - Lane.PointsBegin;
	if (LaneNumPoints <= (int32)MaxPoints)
	{
		// If we can fit all the lane's points, just do a copy.
		NumPoints = (uint8)LaneNumPoints;
		for (int32 Index = 0; Index < (int32)NumPoints; Index++)
		{
			LanePoints[Index] = ZoneGraphStorage.LanePoints[Lane.PointsBegin + Index];
			LaneTangentVectors[Index] = FMassSnorm8Vector2D(FVector2D(ZoneGraphStorage.LaneTangentVectors[Lane.PointsBegin + Index]));
			LanePointProgressions[Index] = FMassInt16Real10(ZoneGraphStorage.LanePointProgressions[Lane.PointsBegin + Index]);
		}
	}
	else
	{
		// Find the segment of the lane that is important and copy that.
		int32 StartSegmentIndex = 0;
		int32 EndSegmentIndex = 0;
		UE::ZoneGraph::Query::CalculateLaneSegmentIndexAtDistance(ZoneGraphStorage, CurrentLaneHandle, StartDistance, StartSegmentIndex);
		UE::ZoneGraph::Query::CalculateLaneSegmentIndexAtDistance(ZoneGraphStorage, CurrentLaneHandle, EndDistance, EndSegmentIndex);

		// Expand if close to start of a segment start.
		if ((StartSegmentIndex - 1) >= Lane.PointsBegin && (StartDistance - InflateDistance) < ZoneGraphStorage.LanePointProgressions[StartSegmentIndex])
		{
			StartSegmentIndex--;
		}
		// Expand if close to end segment end.
		if ((EndSegmentIndex + 1) < (Lane.PointsEnd - 2) && (EndDistance + InflateDistance) > ZoneGraphStorage.LanePointProgressions[EndSegmentIndex + 1])
		{
			EndSegmentIndex++;
		}
	
		NumPoints = (uint8)FMath::Min((EndSegmentIndex - StartSegmentIndex) + 2, (int32)MaxPoints);

		for (int32 Index = 0; Index < (int32)NumPoints; Index++)
		{
			check((StartSegmentIndex + Index) >= Lane.PointsBegin && (StartSegmentIndex + Index) < Lane.PointsEnd);
			LanePoints[Index] = ZoneGraphStorage.LanePoints[StartSegmentIndex + Index];
			LaneTangentVectors[Index] = FMassSnorm8Vector2D(FVector2D(ZoneGraphStorage.LaneTangentVectors[StartSegmentIndex + Index]));
			LanePointProgressions[Index] = FMassInt16Real10(ZoneGraphStorage.LanePointProgressions[StartSegmentIndex + Index]);
		}
	}

	// Calculate extra space around the lane on adjacent lanes.
	TArray<FZoneGraphLinkedLane> LinkedLanes;
	UE::ZoneGraph::Query::GetLinkedLanes(ZoneGraphStorage, CurrentLaneHandle, EZoneLaneLinkType::Adjacent, EZoneLaneLinkFlags::Left|EZoneLaneLinkFlags::Right, EZoneLaneLinkFlags::None, LinkedLanes);

	float AdjacentLeftWidth = 0.0f;
	float AdjacentRightWidth = 0.0f;
	for (const FZoneGraphLinkedLane& LinkedLane : LinkedLanes)
	{
		if (LinkedLane.HasFlags(EZoneLaneLinkFlags::Left))
		{
			const FZoneLaneData& AdjacentLane = ZoneGraphStorage.Lanes[LinkedLane.DestLane.Index];
			AdjacentLeftWidth += AdjacentLane.Width;
		}
		else if (LinkedLane.HasFlags(EZoneLaneLinkFlags::Right))
		{
			const FZoneLaneData& AdjacentLane = ZoneGraphStorage.Lanes[LinkedLane.DestLane.Index];
			AdjacentRightWidth += AdjacentLane.Width;
		}
	}
	LaneLeftSpace = FMassInt16Real(AdjacentLeftWidth);
	LaneRightSpace = FMassInt16Real(AdjacentRightWidth);
}

// Fills Points from CachedLane
bool FMassZoneGraphShortPathFragment::RequestPath(const FMassZoneGraphCachedLaneFragment& CachedLane, const FZoneGraphShortPathRequest& Request, const float InCurrentDistanceAlongLane, const float AgentRadius)
{
	Reset();

	if (CachedLane.NumPoints < 2)
	{
		return false;
	}


	// The current distance can come from a quantized lane distance. Check against quantized bounds, but clamp it to the actual path length when calculating the path.
	static_assert(std::is_same_v<decltype(FMassZoneGraphPathPoint::Distance), FMassInt16Real>, "Assuming FMassZoneGraphPathPoint::Distance is quantized to 10 units.");
	const float LaneLengthQuantized = FMath::CeilToFloat(CachedLane.LaneLength / 10.0f) * 10.0f;

	static constexpr float Epsilon = 0.1f;
	ensureMsgf(InCurrentDistanceAlongLane >= -Epsilon && InCurrentDistanceAlongLane <= (LaneLengthQuantized + Epsilon), TEXT("Current distance %f should be within the lane bounds 0.0 - %f"), InCurrentDistanceAlongLane, LaneLengthQuantized);

	const float CurrentDistanceAlongLane = FMath::Min(InCurrentDistanceAlongLane, CachedLane.LaneLength);
	
	// Set common lane parameters
#if WITH_MASSGAMEPLAY_DEBUG
	DebugLaneHandle = CachedLane.LaneHandle;
#endif

	bMoveReverse = Request.bMoveReverse;
	EndOfPathIntent = Request.EndOfPathIntent;
	bPartialResult = false;

	const float DeflatedLaneHalfWidth = FMath::Max(0.0f, CachedLane.LaneWidth.Get() - AgentRadius) * 0.5f;
	const float DeflatedLaneLeft = DeflatedLaneHalfWidth + CachedLane.LaneLeftSpace.Get();
	const float DeflatedLaneRight = DeflatedLaneHalfWidth + CachedLane.LaneRightSpace.Get();

	const float TargetDistanceAlongLane = FMath::Clamp(Request.TargetDistance, 0.0f, CachedLane.LaneLength);
	const float MinDistanceAlongLane = FMath::Min(CurrentDistanceAlongLane, TargetDistanceAlongLane);
	const float MaxDistanceAlongLane = FMath::Max(CurrentDistanceAlongLane, TargetDistanceAlongLane);
	
	const float TangentSign = Request.bMoveReverse ? -1.0f : 1.0f;

	// Slop factors used when testing if a point is conservatively inside the lane.
	constexpr float OffLaneCapSlop = 10.0f;
	constexpr float OffLaneEdgeSlop = 1.0f;

	// Calculate how the start point relates to the corresponding location on lane.
	FVector StartLanePosition;
	FVector StartLaneTangent;
	CachedLane.GetPointAndTangentAtDistance(CurrentDistanceAlongLane, StartLanePosition, StartLaneTangent);
	float StartDistanceAlongPath = CurrentDistanceAlongLane;

	FVector StartPosition = Request.StartPosition;
	// Calculate start point's relation to the start point location on lane.
	const FVector StartDelta = StartPosition - StartLanePosition;
	const FVector StartLeftDir = FVector::CrossProduct(StartLaneTangent, FVector::UpVector);
	float StartLaneOffset = FloatCastChecked<float>(FVector::DotProduct(StartLeftDir, StartDelta), UE::LWC::DefaultFloatPrecision);
	float StartLaneForwardOffset = FloatCastChecked<float>(FVector::DotProduct(StartLaneTangent, StartDelta) * TangentSign, UE::LWC::DefaultFloatPrecision);
	// The point is off-lane if behind the start, or beyond the boundary.
	const bool bStartOffLane = StartLaneForwardOffset < -OffLaneCapSlop
								|| StartLaneOffset < -(DeflatedLaneRight + OffLaneEdgeSlop)
								|| StartLaneOffset > (DeflatedLaneLeft + OffLaneEdgeSlop);
	StartLaneOffset = FMath::Clamp(StartLaneOffset, -DeflatedLaneRight, DeflatedLaneLeft);

	if (bStartOffLane)
	{
		// The start point was off-lane, move the start location along the lane a bit further to have smoother connection.
		const float StartForwardOffset = FMath::Clamp(Request.AnticipationDistance.Get() + StartLaneForwardOffset, 0.0f, Request.AnticipationDistance.Get());
		StartDistanceAlongPath += StartForwardOffset * TangentSign; // Not clamping this distance intentionally so that the halfway point and clamping later works correctly.
	}

	// Calculate how the end point relates to the corresponding location on lane.
	const bool bHasEndOfPathPoint = Request.bIsEndOfPathPositionSet;
	float EndDistanceAlongPath = TargetDistanceAlongLane;
	FVector EndLanePosition = FVector::ZeroVector;
	FVector EndLaneTangent = FVector::ZeroVector;
	bool bEndOffLane = false;
	float EndLaneOffset = StartLaneOffset;

	if (bHasEndOfPathPoint)
	{
		// Calculate end point's relation to the end point location on lane.
		CachedLane.GetPointAndTangentAtDistance(TargetDistanceAlongLane, EndLanePosition, EndLaneTangent);
		const FVector EndPosition = Request.EndOfPathPosition;
		const FVector EndDelta = EndPosition - EndLanePosition;
		const FVector LeftDir = FVector::CrossProduct(EndLaneTangent, FVector::UpVector);
		EndLaneOffset = FloatCastChecked<float>(FVector::DotProduct(LeftDir, EndDelta), UE::LWC::DefaultFloatPrecision);
		const float EndLaneForwardOffset = FloatCastChecked<float>(FVector::DotProduct(EndLaneTangent, EndDelta) * TangentSign, UE::LWC::DefaultFloatPrecision);
		// The point is off-lane if further than the, or beyond the boundary.
		bEndOffLane = EndLaneForwardOffset > OffLaneCapSlop
						|| EndLaneOffset < -(DeflatedLaneRight + OffLaneEdgeSlop)
						|| EndLaneOffset > (DeflatedLaneLeft + OffLaneEdgeSlop);
		EndLaneOffset = FMath::Clamp(EndLaneOffset, -DeflatedLaneRight, DeflatedLaneLeft);

		// Move the end location along the lane a bit back to have smoother connection.
		const float EndForwardOffset = FMath::Clamp(Request.AnticipationDistance.Get() - EndLaneForwardOffset, 0.0f, Request.AnticipationDistance.Get());
		EndDistanceAlongPath -= EndForwardOffset * TangentSign; // Not clamping this distance intentionally so that the halfway point and clamping later works correctly.
	}

	// Clamp the path move distances to current lane. We use halfway point to split the anticipation in case it gets truncated.
	const float HalfwayDistanceAlongLane = FMath::Clamp((StartDistanceAlongPath + EndDistanceAlongPath) * 0.5f, MinDistanceAlongLane, MaxDistanceAlongLane);

	if (Request.bMoveReverse)
	{
		StartDistanceAlongPath = FMath::Clamp(StartDistanceAlongPath, HalfwayDistanceAlongLane, MaxDistanceAlongLane);
		EndDistanceAlongPath = FMath::Clamp(EndDistanceAlongPath, MinDistanceAlongLane, HalfwayDistanceAlongLane);
	}
	else
	{
		StartDistanceAlongPath = FMath::Clamp(StartDistanceAlongPath, MinDistanceAlongLane, HalfwayDistanceAlongLane);
		EndDistanceAlongPath = FMath::Clamp(EndDistanceAlongPath, HalfwayDistanceAlongLane, MaxDistanceAlongLane);
	}

	// Check if the mid path got clamped away. This can happen if start of end or both are off-mesh, or just a short path.
	const float MidPathMoveDistance = FMath::Abs(EndDistanceAlongPath - StartDistanceAlongPath); 
	const bool bHasMidPath = MidPathMoveDistance > KINDA_SMALL_NUMBER;

	// If end position is not set to a specific location, use proposed offset
	if (!bHasEndOfPathPoint)
	{
		// Slope defines how much the offset can change over the course of the path.
		constexpr float MaxLaneOffsetSlope = 1.0f / 10.0f;
		const float MaxOffset = MidPathMoveDistance * MaxLaneOffsetSlope;
		const float LaneOffset = FMath::Clamp(Request.EndOfPathOffset.Get(), -MaxOffset, MaxOffset);
		EndLaneOffset = FMath::Clamp(EndLaneOffset + LaneOffset, -DeflatedLaneRight, DeflatedLaneLeft);
	}

	// Always add off-lane start point.
	if (bStartOffLane)
	{
		FMassZoneGraphPathPoint& StartPoint = Points[NumPoints++];
		StartPoint.DistanceAlongLane = FMassInt16Real10(CurrentDistanceAlongLane);
		StartPoint.Position = Request.StartPosition;
		StartPoint.Tangent = FMassSnorm8Vector2D(StartLaneTangent * TangentSign);
		StartPoint.bOffLane = true;
		StartPoint.bIsLaneExtrema = false;

		// Update start point to be inside the lane.
		CachedLane.GetPointAndTangentAtDistance(StartDistanceAlongPath, StartLanePosition, StartLaneTangent);
		const FVector LeftDir = FVector::CrossProduct(StartLaneTangent, FVector::UpVector);
		StartPosition = StartLanePosition + LeftDir * StartLaneOffset;

		// Adjust the start point to point towards the first on-lane point.
		const FVector DirToClampedPoint = StartPosition - StartPoint.Position;
		StartPoint.Tangent = FMassSnorm8Vector2D(DirToClampedPoint.GetSafeNormal());
	}

	// The second point is added if there was no off-lane start point, or we have mid path.
	// This ensures that there's always at least one start point, and that no excess points are added if both start & end are off-lane close to each other.
	if (!bStartOffLane || bHasMidPath)
	{
		// Add first on-lane point.
		FMassZoneGraphPathPoint& Point = Points[NumPoints++];
		Point.DistanceAlongLane = FMassInt16Real10(StartDistanceAlongPath);
		Point.Position = StartPosition;
		Point.Tangent = FMassSnorm8Vector2D(StartLaneTangent * TangentSign);
		Point.bOffLane = false;
		Point.bIsLaneExtrema = false;
	}

	// Add in between points.
	const float InvDistanceRange = 1.0f / (EndDistanceAlongPath - StartDistanceAlongPath); // Used for lane offset interpolation. 
	float PrevDistanceAlongLane = StartDistanceAlongPath;

	UE::MassNavigation::ZoneGraphPath::FCachedLaneSegmentIterator SegmentIterator(CachedLane, StartDistanceAlongPath, Request.bMoveReverse);
	while (!SegmentIterator.HasReachedDistance(EndDistanceAlongPath))
	{
		// The segment endpoint is start when moving backwards (i.e. the segment index), and end when moving forwards.
		const int32 CurrentSegmentEndPointIndex = SegmentIterator.CurrentSegment + (SegmentIterator.bMoveReverse ? 0 : 1);
		const float DistanceAlongLane = CachedLane.LanePointProgressions[CurrentSegmentEndPointIndex].Get();

		if (FMath::IsNearlyEqual(PrevDistanceAlongLane, DistanceAlongLane) == false)
		{
			if (NumPoints < MaxPoints)
			{
				const FVector& LanePosition = CachedLane.LanePoints[CurrentSegmentEndPointIndex];
				const FVector LaneTangent = CachedLane.LaneTangentVectors[CurrentSegmentEndPointIndex].GetVector();

				const float LaneOffsetT = (DistanceAlongLane - StartDistanceAlongPath) * InvDistanceRange;
				const float LaneOffset = FMath::Lerp(StartLaneOffset, EndLaneOffset, LaneOffsetT);
				const FVector LeftDir = FVector::CrossProduct(LaneTangent, FVector::UpVector);

				FMassZoneGraphPathPoint& Point = Points[NumPoints++];
				Point.DistanceAlongLane = FMassInt16Real10(DistanceAlongLane);
				Point.Position = LanePosition + LeftDir * LaneOffset;
				Point.Tangent = FMassSnorm8Vector2D(LaneTangent * TangentSign);
				Point.bOffLane = false;
				Point.bIsLaneExtrema = false;

				PrevDistanceAlongLane = DistanceAlongLane;
			}
			else
			{
				bPartialResult = true;
				break;
			}
		}
		
		SegmentIterator.Next();
	}

	// The second last point is added if there is no end point, or we have mid path.
	// This ensures that there's always at least one end point, and that no excess points are added if both start & end are off-lane close to each other.
	if (!bHasEndOfPathPoint || bHasMidPath)
	{
		if (NumPoints < MaxPoints)
		{
			// Interpolate last point on mid path.
			FVector LanePosition;
			FVector LaneTangent;
			CachedLane.InterpolatePointAndTangentOnSegment(SegmentIterator.CurrentSegment, EndDistanceAlongPath, LanePosition, LaneTangent);

			const float LaneOffset = EndLaneOffset;
			const FVector LeftDir = FVector::CrossProduct(LaneTangent, FVector::UpVector);

			FMassZoneGraphPathPoint& Point = Points[NumPoints++];
			Point.DistanceAlongLane = FMassInt16Real10(EndDistanceAlongPath);
			Point.Position = LanePosition + LeftDir * LaneOffset;
			Point.Tangent = FMassSnorm8Vector2D(LaneTangent * TangentSign);
			Point.bOffLane = false;
			Point.bIsLaneExtrema = !Request.bIsEndOfPathPositionSet && CachedLane.IsDistanceAtLaneExtrema(EndDistanceAlongPath);
		}
		else
		{
			bPartialResult = true;
		}
	}

	checkf(NumPoints >= 1, TEXT("Path should have at least 1 point at this stage but has none."));

	// Add end of path point if set.
	if (bHasEndOfPathPoint)
	{
		if (NumPoints < MaxPoints)
		{
			const FVector EndPosition = Request.EndOfPathPosition;

			// Use provided direction if set, otherwise use direction from last point on lane to end of path point
			const FVector EndDirection = (Request.bIsEndOfPathDirectionSet) ?
				Request.EndOfPathDirection.Get() :
				(EndPosition - Points[NumPoints-1].Position).GetSafeNormal();
			
			FMassZoneGraphPathPoint& Point = Points[NumPoints++];
			Point.DistanceAlongLane = FMassInt16Real10(TargetDistanceAlongLane);
			Point.Position = EndPosition;
			Point.Tangent = FMassSnorm8Vector2D(EndDirection);
			Point.bOffLane = bEndOffLane;
			Point.bIsLaneExtrema = false;
		}
		else
		{
			bPartialResult = true;
		}
	}

	checkf(NumPoints >= 2, TEXT("Path should have at least 2 points at this stage, has %d."), NumPoints);

	// Calculate movement distance at each point.
	float PathDistance = 0.0f;
	Points[0].Distance.Set(PathDistance);
	for (uint8 PointIndex = 1; PointIndex < NumPoints; PointIndex++)
	{
		FMassZoneGraphPathPoint& PrevPoint = Points[PointIndex - 1];
		FMassZoneGraphPathPoint& Point = Points[PointIndex];
		const FVector PrevPosition = PrevPoint.Position;
		const FVector Position = Point.Position;
		const float DeltaDistance = FloatCastChecked<float>(FVector::Dist(PrevPosition, Position), UE::LWC::DefaultFloatPrecision);
		PathDistance += DeltaDistance;
		Point.Distance.Set(PathDistance);
	}
	
	// If the last point on path reaches end of the lane, set the next handle to the next lane. It will be update when path finishes.
	if (!bPartialResult && Request.NextLaneHandle.IsValid())
	{
		const FMassZoneGraphPathPoint& LastPoint = Points[NumPoints - 1];
	
		if (Request.NextExitLinkType == EZoneLaneLinkType::Adjacent || LastPoint.bIsLaneExtrema)
		{
			NextLaneHandle = Request.NextLaneHandle;
			NextExitLinkType = Request.NextExitLinkType;
		}
	}

	return true;
}

bool FMassZoneGraphShortPathFragment::RequestStand(const FMassZoneGraphCachedLaneFragment& CachedLane, const float CurrentDistanceAlongLane, const FVector& CurrentPosition)
{
	Reset();

	if (CachedLane.NumPoints < 2)
	{
		return false;
	}

	static constexpr float Epsilon = 0.1f;
	check(CurrentDistanceAlongLane >= -Epsilon && CurrentDistanceAlongLane <= (CachedLane.LaneLength + Epsilon));
	
	// Get current location
	FVector CurrentLanePosition;
	FVector CurrentLaneTangent;
	CachedLane.GetPointAndTangentAtDistance(CurrentDistanceAlongLane, CurrentLanePosition, CurrentLaneTangent);
	
	// Set common lane parameters
#if WITH_MASSGAMEPLAY_DEBUG
	DebugLaneHandle = CachedLane.LaneHandle;
#endif
	bMoveReverse = false;
	EndOfPathIntent = EMassMovementAction::Stand;
	bPartialResult = false;

	// Add start point, if the start is outside the lane, add another point to get back to lane.
	const FVector StartMoveOffset = CurrentPosition - CurrentLanePosition;

	FMassZoneGraphPathPoint& StartPoint = Points[NumPoints];
	StartPoint.DistanceAlongLane = FMassInt16Real10(CurrentDistanceAlongLane);
	StartPoint.Position = CurrentPosition;
	StartPoint.Tangent = FMassSnorm8Vector2D(CurrentLaneTangent);
	StartPoint.bOffLane = false;
	StartPoint.bIsLaneExtrema = false;
	StartPoint.Distance = FMassInt16Real(0.0f);
	NumPoints++;

	FMassZoneGraphPathPoint& EndPoint = Points[NumPoints];
	EndPoint.DistanceAlongLane = FMassInt16Real10(CurrentDistanceAlongLane);
	EndPoint.Position = CurrentPosition;
	EndPoint.Tangent = FMassSnorm8Vector2D(CurrentLaneTangent);
	EndPoint.bOffLane = false;
	EndPoint.bIsLaneExtrema = false;
	EndPoint.Distance = FMassInt16Real(0.0f);
	NumPoints++;

	return true;
}
