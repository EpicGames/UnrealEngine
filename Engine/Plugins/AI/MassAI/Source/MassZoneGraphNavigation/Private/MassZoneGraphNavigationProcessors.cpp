// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassZoneGraphNavigationProcessors.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassNavigationTypes.h"
#include "MassNavigationFragments.h"
#include "MassNavigationUtils.h"
#include "Avoidance/MassAvoidanceFragments.h"
#include "Avoidance/MassAvoidanceProcessors.h"
#include "MassCommonFragments.h"
#include "MassExecutionContext.h"
#include "MassSignalSubsystem.h"
#include "ZoneGraphSubsystem.h"
#include "ZoneGraphQuery.h"
#include "VisualLogger/VisualLogger.h"
#include "MassSimulationLOD.h"
#include "Engine/World.h"
#include "MassDebugger.h"
#include "MassNavigationDebug.h"
#include <limits>

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphNavigationProcessors)

#define WITH_ZONEGRAPH_DEBUG_DETAILED 0

//----------------------------------------------------------------------//
//  UMassZoneGraphLocationInitializer
//----------------------------------------------------------------------//
UMassZoneGraphLocationInitializer::UMassZoneGraphLocationInitializer()
	: EntityQuery(*this)
{
	ObservedTypes.Add(FMassZoneGraphLaneLocationFragment::StaticStruct());
	ObservedOperations = EMassObservedOperationFlags::Add;
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
}

void UMassZoneGraphLocationInitializer::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FTransformFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadWrite); // Make optional?
	EntityQuery.AddConstSharedRequirement<FMassZoneGraphNavigationParameters>(EMassFragmentPresence::All);
	EntityQuery.AddSubsystemRequirement<UZoneGraphSubsystem>(EMassFragmentAccess::ReadOnly);
}

void UMassZoneGraphLocationInitializer::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	EntityQuery.ForEachEntityChunk(Context, [this](FMassExecutionContext& Context)
	{
		const UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetSubsystemChecked<UZoneGraphSubsystem>();

		const TArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList = Context.GetMutableFragmentView<FMassZoneGraphLaneLocationFragment>();
		const TArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetMutableFragmentView<FMassMoveTargetFragment>();
		const TConstArrayView<FTransformFragment> TransformList = Context.GetFragmentView<FTransformFragment>();
		const FMassZoneGraphNavigationParameters& NavigationParams = Context.GetConstSharedFragment<FMassZoneGraphNavigationParameters>();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FTransformFragment& Transform = TransformList[EntityIt];
			const FVector& AgentLocation = Transform.GetTransform().GetLocation();
			FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
			FMassZoneGraphLaneLocationFragment& LaneLocation = LaneLocationList[EntityIt];

			const FVector QuerySize(NavigationParams.QueryRadius);
			const FBox QueryBounds(AgentLocation - QuerySize, AgentLocation + QuerySize);
			
			FZoneGraphLaneLocation NearestLane;
			float NearestLaneDistSqr = 0;
			
			if (ZoneGraphSubsystem.FindNearestLane(QueryBounds, NavigationParams.LaneFilter, NearestLane, NearestLaneDistSqr))
			{
				const FZoneGraphStorage* ZoneGraphStorage = ZoneGraphSubsystem.GetZoneGraphStorage(NearestLane.LaneHandle.DataHandle);
				check(ZoneGraphStorage); // Assume valid storage since we just got result.

				LaneLocation.LaneHandle = NearestLane.LaneHandle;
				LaneLocation.DistanceAlongLane = NearestLane.DistanceAlongLane;
				UE::ZoneGraph::Query::GetLaneLength(*ZoneGraphStorage, LaneLocation.LaneHandle, LaneLocation.LaneLength);
				
				MoveTarget.Center = AgentLocation;
				MoveTarget.Forward = NearestLane.Tangent;
			}
			else
			{
				LaneLocation.LaneHandle.Reset();
				LaneLocation.DistanceAlongLane = 0.0f;
				LaneLocation.LaneLength = 0.0f;

				MoveTarget.Center = AgentLocation;
				MoveTarget.Forward = FVector::ForwardVector;
			}

			MoveTarget.DistanceToGoal = 0.0f;
			MoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
			MoveTarget.SlackRadius = 0.0f;
		}
	});
}


//----------------------------------------------------------------------//
//  UMassZoneGraphPathFollowProcessor
//----------------------------------------------------------------------//
UMassZoneGraphPathFollowProcessor::UMassZoneGraphPathFollowProcessor()
	: EntityQuery_Conditional(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;
	ExecutionOrder.ExecuteInGroup = UE::Mass::ProcessorGroupNames::Tasks;
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Avoidance);
}

void UMassZoneGraphPathFollowProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	ProcessorRequirements.AddSubsystemRequirement<UMassSignalSubsystem>(EMassFragmentAccess::ReadWrite);

	EntityQuery_Conditional.AddRequirement<FMassZoneGraphShortPathFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery_Conditional.AddRequirement<FMassSimulationLODFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.AddRequirement<FMassSimulationVariableTickFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);

	EntityQuery_Conditional.AddChunkRequirement<FMassSimulationVariableTickChunkFragment>(EMassFragmentAccess::ReadOnly, EMassFragmentPresence::Optional);
	EntityQuery_Conditional.SetChunkFilter(&FMassSimulationVariableTickChunkFragment::ShouldTickChunkThisFrame);

	EntityQuery_Conditional.AddSubsystemRequirement<UZoneGraphSubsystem>(EMassFragmentAccess::ReadOnly);

#if WITH_MASSGAMEPLAY_DEBUG
	EntityQuery_Conditional.DebugEnableEntityOwnerLogging();
#endif
}

void UMassZoneGraphPathFollowProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	UMassSignalSubsystem* SignalSubsystem = Context.GetMutableSubsystem<UMassSignalSubsystem>();
	if (!SignalSubsystem)
	{
		return;
	}

	UWorld* DebugDrawWorld = UE::Mass::Debug::GetDebugDrawWorld(Context);
	
	TArray<FMassEntityHandle> EntitiesToSignalPathDone;
	TArray<FMassEntityHandle> EntitiesToSignalLaneChanged;

	EntityQuery_Conditional.ForEachEntityChunk(Context, [FallbackLogOwner = this, &EntitiesToSignalPathDone, &EntitiesToSignalLaneChanged, DebugDrawWorld](FMassExecutionContext& Context)
	{
		const UZoneGraphSubsystem& ZoneGraphSubsystem = Context.GetSubsystemChecked<UZoneGraphSubsystem>();

		const TArrayView<FMassZoneGraphShortPathFragment> ShortPathList = Context.GetMutableFragmentView<FMassZoneGraphShortPathFragment>();
		const TArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList = Context.GetMutableFragmentView<FMassZoneGraphLaneLocationFragment>();
		const TArrayView<FMassMoveTargetFragment> MoveTargetList = Context.GetMutableFragmentView<FMassMoveTargetFragment>();
		const TConstArrayView<FMassSimulationLODFragment> SimLODList = Context.GetFragmentView<FMassSimulationLODFragment>();
		const bool bHasLOD = (SimLODList.Num() > 0);
		const TConstArrayView<FMassSimulationVariableTickFragment> SimVariableTickList = Context.GetFragmentView<FMassSimulationVariableTickFragment>();
		const bool bHasVariableTick = (SimVariableTickList.Num() > 0);
		const float WorldDeltaTime = Context.GetDeltaTimeSeconds();

		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			FMassZoneGraphShortPathFragment& ShortPath = ShortPathList[EntityIt];
			FMassZoneGraphLaneLocationFragment& LaneLocation = LaneLocationList[EntityIt];
			FMassMoveTargetFragment& MoveTarget = MoveTargetList[EntityIt];
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);
			const float DeltaTime = bHasVariableTick ? SimVariableTickList[EntityIt].DeltaTime : WorldDeltaTime;

			bool bDisplayDebug = false;
			const UObject* LogOwner = nullptr;
#if WITH_MASSGAMEPLAY_DEBUG // this will result in bDisplayDebug == false and disabling of all the vlogs below
			const UE::MassNavigation::Debug::FDebugContext NavigationDebugContext(Context, FallbackLogOwner, LogMassNavigation, DebugDrawWorld, Entity, EntityIt);
			bDisplayDebug = NavigationDebugContext.ShouldLogEntity();
			LogOwner = NavigationDebugContext.GetLogOwner();
			if (bDisplayDebug)
			{
				UE_VLOG(LogOwner, LogMassNavigation, Log, TEXT("Entity [%s] Updating path following"), *Entity.DebugGetDescription());
			}
#endif // WITH_MASSGAMEPLAY_DEBUG

			// Must have at least two points to interpolate.
			if (MoveTarget.GetCurrentAction() == EMassMovementAction::Move && ShortPath.NumPoints >= 2)
			{
				const bool bWasDone = ShortPath.IsDone();

				// Note: this should be in sync with the logic in apply velocity.
				const bool bHasSteering = (bHasLOD == false) || (SimLODList[EntityIt].LOD != EMassLOD::Off);

				if (!bHasSteering || !MoveTarget.bSteeringFallingBehind)
				{
					// Update progress
					ShortPath.ProgressDistance += MoveTarget.DesiredSpeed.Get() * DeltaTime;
				}

				// @todo MassMovement: Ideally we would carry over any left over distance to the next path, especially when dealing with larger timesteps.
				// @todo MassMovement: Feedback current movement progress back to ShortPath.DesiredSpeed.

				if (!bWasDone)
				{
					const uint8 LastPointIndex = ShortPath.NumPoints - 1;
#if WITH_MASSGAMEPLAY_DEBUG
					ensureMsgf(LaneLocation.LaneHandle == ShortPath.DebugLaneHandle, TEXT("Short path lane should match current lane location."));
#endif // WITH_MASSGAMEPLAY_DEBUG

					if (ShortPath.ProgressDistance <= 0.0f)
					{
						// Requested time before the start of the path
						LaneLocation.DistanceAlongLane = ShortPath.Points[0].DistanceAlongLane.Get();
						
						MoveTarget.Center = ShortPath.Points[0].Position;
						MoveTarget.Forward = ShortPath.Points[0].Tangent.GetVector();
						MoveTarget.DistanceToGoal = ShortPath.Points[LastPointIndex].Distance.Get();
						MoveTarget.bOffBoundaries = ShortPath.Points[0].bOffLane;

						UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Verbose, TEXT("Entity [%s] before start of lane %s at distance %.1f. Distance to goal: %.1f. Off Boundaries: %s"),
							*Entity.DebugGetDescription(),
							*LaneLocation.LaneHandle.ToString(),
							LaneLocation.DistanceAlongLane,
							MoveTarget.DistanceToGoal,
							*LexToString((bool)MoveTarget.bOffBoundaries));
					}
					else if (ShortPath.ProgressDistance <= ShortPath.Points[LastPointIndex].Distance.Get())
					{
						// Requested time along the path, interpolate.
						uint8 PointIndex = 0;
						while (PointIndex < (ShortPath.NumPoints - 2))
						{
							const FMassZoneGraphPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];
							if (ShortPath.ProgressDistance <= NextPoint.Distance.Get())
							{
								break;
							}
							PointIndex++;
						}

						const FMassZoneGraphPathPoint& CurrPoint = ShortPath.Points[PointIndex];
						const FMassZoneGraphPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];
						const float T = (ShortPath.ProgressDistance - CurrPoint.Distance.Get()) / (NextPoint.Distance.Get() - CurrPoint.Distance.Get());
						
						LaneLocation.DistanceAlongLane = FMath::Min(FMath::Lerp(CurrPoint.DistanceAlongLane.Get(), NextPoint.DistanceAlongLane.Get(), T), LaneLocation.LaneLength);

						MoveTarget.Center = FMath::Lerp(CurrPoint.Position, NextPoint.Position, T);
						MoveTarget.Forward = FMath::Lerp(CurrPoint.Tangent.GetVector(), NextPoint.Tangent.GetVector(), T).GetSafeNormal();
						MoveTarget.DistanceToGoal = ShortPath.Points[LastPointIndex].Distance.Get() - FMath::Lerp(CurrPoint.Distance.Get(), NextPoint.Distance.Get(), T);
						MoveTarget.bOffBoundaries = CurrPoint.bOffLane || NextPoint.bOffLane;

						UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Verbose, TEXT("Entity [%s] along lane %s at distance %.1f. Distance to goal: %.1f. Off Boundaries: %s"),
							*Entity.DebugGetDescription(),
							*LaneLocation.LaneHandle.ToString(),
							LaneLocation.DistanceAlongLane,
							MoveTarget.DistanceToGoal,
							*LexToString((bool)MoveTarget.bOffBoundaries));
					}
					else
					{
						// Requested time after the end of the path, clamp to lane length in case quantization overshoots.
						LaneLocation.DistanceAlongLane = FMath::Min(ShortPath.Points[LastPointIndex].DistanceAlongLane.Get(), LaneLocation.LaneLength);

						MoveTarget.Center = ShortPath.Points[LastPointIndex].Position;
						MoveTarget.Forward = ShortPath.Points[LastPointIndex].Tangent.GetVector();
						MoveTarget.DistanceToGoal = 0.0f;
						MoveTarget.bOffBoundaries = ShortPath.Points[LastPointIndex].bOffLane;

						UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Log, TEXT("Entity [%s] Finished path follow on lane %s at distance %f. Off Boundaries: %s"),
							*Entity.DebugGetDescription(), *LaneLocation.LaneHandle.ToString(), LaneLocation.DistanceAlongLane, *LexToString((bool)MoveTarget.bOffBoundaries));

						if (bDisplayDebug)
						{
							UE_VLOG(LogOwner, LogMassNavigation, Log, TEXT("Entity [%s] End of path."), *Entity.DebugGetDescription());
						}

						// Check to see if need advance to next lane.
						if (ShortPath.NextLaneHandle.IsValid())
						{
							const FZoneGraphStorage* ZoneGraphStorage = ZoneGraphSubsystem.GetZoneGraphStorage(LaneLocation.LaneHandle.DataHandle);
							if (ZoneGraphStorage != nullptr)
							{
								if (ShortPath.NextExitLinkType == EZoneLaneLinkType::Outgoing)
								{
									float NewLaneLength = 0.f;
									UE::ZoneGraph::Query::GetLaneLength(*ZoneGraphStorage, ShortPath.NextLaneHandle, NewLaneLength);

									UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Log, TEXT("Entity [%s] Switching to OUTGOING lane %s -> %s, new distance %f."),
										*Entity.DebugGetDescription(), *LaneLocation.LaneHandle.ToString(), *ShortPath.NextLaneHandle.ToString(), 0.f);

									// update lane location
									LaneLocation.LaneHandle = ShortPath.NextLaneHandle;
									LaneLocation.LaneLength = NewLaneLength;
									LaneLocation.DistanceAlongLane = 0.0f;
								}
								else if (ShortPath.NextExitLinkType == EZoneLaneLinkType::Incoming)
								{
									float NewLaneLength = 0.f;
									UE::ZoneGraph::Query::GetLaneLength(*ZoneGraphStorage, ShortPath.NextLaneHandle, NewLaneLength);

									UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Log, TEXT("Entity [%s] Switching to INCOMING lane %s -> %s, new distance %f."),
										*Entity.DebugGetDescription(), *LaneLocation.LaneHandle.ToString(), *ShortPath.NextLaneHandle.ToString(), NewLaneLength);

									// update lane location
									LaneLocation.LaneHandle = ShortPath.NextLaneHandle;
									LaneLocation.LaneLength = NewLaneLength;
									LaneLocation.DistanceAlongLane = NewLaneLength;
								}
								else if (ShortPath.NextExitLinkType == EZoneLaneLinkType::Adjacent)
								{
									FZoneGraphLaneLocation NewLocation;
									float DistanceSqr;
									if (UE::ZoneGraph::Query::FindNearestLocationOnLane(*ZoneGraphStorage, ShortPath.NextLaneHandle,  MoveTarget.Center, std::numeric_limits<float>::infinity(), NewLocation, DistanceSqr))
									{
										float NewLaneLength = 0.f;
										UE::ZoneGraph::Query::GetLaneLength(*ZoneGraphStorage, ShortPath.NextLaneHandle, NewLaneLength);

										UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Log, TEXT("Entity [%s] Switching to ADJACENT lane %s -> %s, new distance %f."),
											*Entity.DebugGetDescription(), *LaneLocation.LaneHandle.ToString(), *ShortPath.NextLaneHandle.ToString(), NewLocation.DistanceAlongLane);

										// update lane location
										LaneLocation.LaneHandle = ShortPath.NextLaneHandle;
										LaneLocation.LaneLength = NewLaneLength;
										LaneLocation.DistanceAlongLane = NewLocation.DistanceAlongLane;

										MoveTarget.Forward = NewLocation.Tangent;
									}
									else
									{
										UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Error, TEXT("Entity [%s] Failed to switch to ADJACENT lane %s -> %s."),
											*Entity.DebugGetDescription(), *LaneLocation.LaneHandle.ToString(), *ShortPath.NextLaneHandle.ToString());
									}
								}
								else
								{
									ensureMsgf(false, TEXT("Unhandle NextExitLinkType type %s"), *UEnum::GetValueAsString(ShortPath.NextExitLinkType));
								}

								// Signal lane changed.
								EntitiesToSignalLaneChanged.Add(Entity);
							}
							else
							{
								UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Error, TEXT("Entity [%s] Could not find ZoneGraph storage for lane %s."),
									*Entity.DebugGetDescription(), *LaneLocation.LaneHandle.ToString());
							}
						}
						else
						{
							UE_CVLOG(bDisplayDebug, LogOwner, LogMassNavigation, Log, TEXT("Entity [%s] Next lane not defined."), *Entity.DebugGetDescription());
						}

						ShortPath.bDone = true;
					}
				}

				const bool bIsDone = ShortPath.IsDone();

				// Signal path done.
				if (!bWasDone && bIsDone)
				{
					EntitiesToSignalPathDone.Add(Entity);
				}

#if WITH_MASSGAMEPLAY_DEBUG
				if (bDisplayDebug)
				{
					const FColor EntityColor = UE::Mass::Debug::GetEntityDebugColor(Entity);

					const FVector ZOffset(0,0,25);
					const FColor LightEntityColor = UE::MassNavigation::Debug::MixColors(EntityColor, FColor::White);
					
					for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints - 1; PointIndex++)
					{
						const FMassZoneGraphPathPoint& CurrPoint = ShortPath.Points[PointIndex];
						const FMassZoneGraphPathPoint& NextPoint = ShortPath.Points[PointIndex + 1];

						// Path
						UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Display, CurrPoint.Position + ZOffset, NextPoint.Position + ZOffset, EntityColor, /*Thickness*/3, TEXT(""));
					}
					
					for (uint8 PointIndex = 0; PointIndex < ShortPath.NumPoints; PointIndex++)
					{
						const FMassZoneGraphPathPoint& CurrPoint = ShortPath.Points[PointIndex];
						const FVector CurrBase = CurrPoint.Position + ZOffset;
						// Lane tangents
						UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Display, CurrBase, CurrBase + CurrPoint.Tangent.GetVector() * 100.0f, LightEntityColor, /*Thickness*/1, TEXT(""));
					}

					if (ShortPath.NumPoints > 0 && ShortPath.NextLaneHandle.IsValid())
					{
						const FMassZoneGraphPathPoint& LastPoint = ShortPath.Points[ShortPath.NumPoints - 1];
						const FVector CurrBase = LastPoint.Position + ZOffset;
						UE_VLOG_SEGMENT_THICK(LogOwner, LogMassNavigation, Display, CurrBase, CurrBase + FVector(0,0,100), FColor::Red, /*Thickness*/3, TEXT("Next: %s"), *ShortPath.NextLaneHandle.ToString());
					}
				}
#endif // WITH_MASSGAMEPLAY_DEBUG
			}
		}
	});

	if (EntitiesToSignalPathDone.Num())
	{
		SignalSubsystem->SignalEntities(UE::Mass::Signals::FollowPointPathDone, EntitiesToSignalPathDone);
	}
	if (EntitiesToSignalLaneChanged.Num())
	{
		SignalSubsystem->SignalEntities(UE::Mass::Signals::CurrentLaneChanged, EntitiesToSignalLaneChanged);
	}
}



//----------------------------------------------------------------------//
//  UMassZoneGraphLaneCacheBoundaryProcessor
//----------------------------------------------------------------------//
UMassZoneGraphLaneCacheBoundaryProcessor::UMassZoneGraphLaneCacheBoundaryProcessor()
	: EntityQuery(*this)
{
	ExecutionFlags = (int32)EProcessorExecutionFlags::AllNetModes;

	bAutoRegisterWithProcessingPhases = true;
	ExecutionOrder.ExecuteAfter.Add(UE::Mass::ProcessorGroupNames::LOD);
	ExecutionOrder.ExecuteBefore.Add(UE::Mass::ProcessorGroupNames::Avoidance);
}

void UMassZoneGraphLaneCacheBoundaryProcessor::ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager)
{
	EntityQuery.AddRequirement<FMassZoneGraphCachedLaneFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassMoveTargetFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassZoneGraphLaneLocationFragment>(EMassFragmentAccess::ReadOnly);
	EntityQuery.AddRequirement<FMassLaneCacheBoundaryFragment>(EMassFragmentAccess::ReadWrite);
	EntityQuery.AddRequirement<FMassNavigationEdgesFragment>(EMassFragmentAccess::ReadWrite);	// output edges
	EntityQuery.AddTagRequirement<FMassOffLODTag>(EMassFragmentPresence::None);

#if WITH_MASSGAMEPLAY_DEBUG
	EntityQuery.DebugEnableEntityOwnerLogging();
#endif
}

void UMassZoneGraphLaneCacheBoundaryProcessor::Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context)
{
	QUICK_SCOPE_CYCLE_COUNTER(MassLaneCacheBoundaryProcessor);

	UWorld* DebugDrawWorld = UE::Mass::Debug::GetDebugDrawWorld(Context);

	EntityQuery.ForEachEntityChunk(Context, [FallbackLogOwner = this, DebugDrawWorld](FMassExecutionContext& Context)
	{
		TConstArrayView<FMassZoneGraphCachedLaneFragment> CachedLaneList = Context.GetFragmentView<FMassZoneGraphCachedLaneFragment>();
		TConstArrayView<FMassZoneGraphLaneLocationFragment> LaneLocationList = Context.GetFragmentView<FMassZoneGraphLaneLocationFragment>();
		TConstArrayView<FMassMoveTargetFragment> MovementTargetList = Context.GetFragmentView<FMassMoveTargetFragment>();
		TArrayView<FMassLaneCacheBoundaryFragment> LaneCacheBoundaryList = Context.GetMutableFragmentView<FMassLaneCacheBoundaryFragment>();
		TArrayView<FMassNavigationEdgesFragment> EdgesList = Context.GetMutableFragmentView<FMassNavigationEdgesFragment>();

		TArray<FZoneGraphLinkedLane> LinkedLanes;
		LinkedLanes.Reserve(4);	
	
		for (FMassExecutionContext::FEntityIterator EntityIt = Context.CreateEntityIterator(); EntityIt; ++EntityIt)
		{
			const FMassZoneGraphCachedLaneFragment& CachedLane = CachedLaneList[EntityIt];
			const FMassZoneGraphLaneLocationFragment& LaneLocation = LaneLocationList[EntityIt];
			const FMassMoveTargetFragment& MovementTarget = MovementTargetList[EntityIt];
			FMassNavigationEdgesFragment& Edges = EdgesList[EntityIt];
			FMassLaneCacheBoundaryFragment& LaneCacheBoundary = LaneCacheBoundaryList[EntityIt];
			const FMassEntityHandle Entity = Context.GetEntity(EntityIt);

			// First check if we moved enough for an update
			const FVector::FReal DeltaDistSquared = FVector::DistSquared(MovementTarget.Center, LaneCacheBoundary.LastUpdatePosition);
			const FVector::FReal UpdateDistanceThresholdSquared = FMath::Square(50.);

#if WITH_MASSGAMEPLAY_DEBUG && WITH_ZONEGRAPH_DEBUG_DETAILED
			const UE::MassNavigation::Debug::FDebugContext ObstacleDebugContext(Context, FallbackLogOwner, LogAvoidanceObstacles, DebugDrawWorld, Entity, EntityIt);
			if (ObstacleDebugContext.ShouldLogEntity())
			{
				DebugDrawSphere(ObstacleDebugContext, LaneCacheBoundary.LastUpdatePosition, /*Radius=*/10.f, FColor(128,128,128));
				DebugDrawSphere(ObstacleDebugContext, MovementTarget.Center, /*Radius=*/10.f, FColor(255,255,255));
			}
#endif
	
			if (DeltaDistSquared < UpdateDistanceThresholdSquared && CachedLane.CacheID == LaneCacheBoundary.LastUpdateCacheID)
			{
				// Not moved enough
				continue;
			}

			LaneCacheBoundary.LastUpdatePosition = MovementTarget.Center;
			LaneCacheBoundary.LastUpdateCacheID = CachedLane.CacheID;

			// If we are skipping the update we don't want to reset the edges, we just want to execute up to the display of the lane.
			Edges.AvoidanceEdges.Reset();
			if (CachedLane.NumPoints < 2)
			{
				// Nothing to do
				continue;
			}
			

#if WITH_MASSGAMEPLAY_DEBUG && WITH_ZONEGRAPH_DEBUG_DETAILED
			DebugDrawSphere(ObstacleDebugContext, MovementTarget.Center, /*Radius=*/100.f, FColor(128,128,128));
#endif // WITH_MASSGAMEPLAY_DEBUG

			const float HalfWidth = 0.5f * CachedLane.LaneWidth.Get();

			static const int32 MaxPoints = 4;
			FVector Points[MaxPoints];
			FVector SegmentDirections[MaxPoints];
			FVector SegmentNormals[MaxPoints];
			FVector MiterDirections[MaxPoints];

			const int32 CurrentSegment = CachedLane.FindSegmentIndexAtDistance(LaneLocation.DistanceAlongLane);
			const int32 FirstSegment = FMath::Max(0, CurrentSegment - 1); // Segment should always be <= CachedLane.NumPoints - 2
			const int32 LastSegment = FMath::Min(CurrentSegment + 1, (int32)CachedLane.NumPoints - 2); // CachedLane.NumPoints - 1 is the lane last point, CachedLane.NumPoints - 2 is the lane last segment
			const int32 NumPoints = (LastSegment - FirstSegment + 1) + 1; // NumPoint = NumSegment + 1
			check(NumPoints >= 2);
			check(NumPoints <= MaxPoints);
			
			// Get points
			for (int32 Index = 0; Index < NumPoints; Index++)
			{
				Points[Index] = CachedLane.LanePoints[FirstSegment + Index];
			}
			
			// Calculate segment direction and normal. Normal points to left, away from the segment.  
			for (int32 Index = 0; Index < NumPoints - 1; Index++)
			{
				SegmentDirections[Index] = (Points[Index + 1] - Points[Index]).GetSafeNormal();
				SegmentNormals[Index] = UE::MassNavigation::GetLeftDirection(SegmentDirections[Index], FVector::UpVector);
			}

			// Last point inherits the direction from the last segment.
			SegmentDirections[NumPoints - 1] = SegmentDirections[NumPoints - 2];
			SegmentNormals[NumPoints - 1] = SegmentNormals[NumPoints - 2];

			// Calculate miter directions at inner corners.
			// Note, mitered direction is average of the adjacent edge left directions, and scaled so that the expanded edges are parallel to the stem.
			// First and last point dont have adjacent segments, and not mitered.
			MiterDirections[0] = SegmentNormals[0];
			MiterDirections[NumPoints - 1] = SegmentNormals[NumPoints - 1];
			for (int32 Index = 1; Index < NumPoints - 1; Index++)
			{
				MiterDirections[Index] = UE::MassNavigation::ComputeMiterNormal(SegmentNormals[Index - 1], SegmentNormals[Index]);
			}

			// Compute left and right positions from lane width and miter directions.
			const float LeftWidth = HalfWidth + CachedLane.LaneLeftSpace.Get();
			const float RightWidth = HalfWidth + CachedLane.LaneRightSpace.Get();
			FVector LeftPositions[MaxPoints];
			FVector RightPositions[MaxPoints];
			for (int32 Index = 0; Index < NumPoints; Index++)
			{
				const FVector MiterDir = MiterDirections[Index];
				LeftPositions[Index] = Points[Index] + LeftWidth * MiterDir;
				RightPositions[Index] = Points[Index] - RightWidth * MiterDir;
			}
			int32 NumLeftPositions = NumPoints;
			int32 NumRightPositions = NumPoints;


#if WITH_MASSGAMEPLAY_DEBUG && WITH_ZONEGRAPH_DEBUG_DETAILED
			if (ObstacleDebugContext.ShouldLogEntity())
			{
				float Radius = 2.f;
				for (int32 Index = 0; Index < NumPoints; Index++)
				{
					if (Index < NumPoints - 1)
					{
						DebugDrawLine(ObstacleDebugContext, Points[Index], Points[Index + 1], FColor::Blue, /*Thickness=*/6.f);
					}
					DebugDrawSphere(ObstacleDebugContext, Points[Index], Radius, FColor::Blue);
					DebugDrawSphere(ObstacleDebugContext, LeftPositions[Index], Radius, FColor::Green);
					DebugDrawSphere(ObstacleDebugContext, RightPositions[Index], Radius, FColor::Red);
					Radius += 4.f;
				}
			}
#endif //WITH_MASSGAMEPLAY_DEBUG

			// Remove edges crossing when there are 3 edges.
			if (NumPoints == 4)
			{
				FVector Intersection = FVector::ZeroVector;
				if (FMath::SegmentIntersection2D(LeftPositions[0], LeftPositions[1], LeftPositions[2], LeftPositions[3], Intersection))
				{
					LeftPositions[1] = Intersection;
					LeftPositions[2] = LeftPositions[3];
					NumLeftPositions--;
				}

				Intersection = FVector::ZeroVector;
				if (FMath::SegmentIntersection2D(RightPositions[0], RightPositions[1], RightPositions[2], RightPositions[3], Intersection))
				{
					RightPositions[1] = Intersection;
					RightPositions[2] = RightPositions[3];
					NumRightPositions--;
				}
			}

			// Add edges
			for (int32 Index = 0; Index < NumLeftPositions - 1; Index++)
			{
				Edges.AvoidanceEdges.Add(FNavigationAvoidanceEdge(LeftPositions[Index + 1], LeftPositions[Index])); // Left side: reverse start and end to keep the normal inside.
			}

			for (int32 Index = 0; Index < NumRightPositions - 1; Index++)
			{
				Edges.AvoidanceEdges.Add(FNavigationAvoidanceEdge(RightPositions[Index], RightPositions[Index + 1]));
			}
		}
	});
}
