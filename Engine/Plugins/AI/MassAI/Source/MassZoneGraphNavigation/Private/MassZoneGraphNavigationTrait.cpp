// Copyright Epic Games, Inc. All Rights Reserved.
#include "MassZoneGraphNavigationTrait.h"
#include "MassEntityTemplateRegistry.h"
#include "MassCommonFragments.h"
#include "MassMovementFragments.h"
#include "MassNavigationFragments.h"
#include "MassZoneGraphNavigationFragments.h"
#include "Engine/World.h"
#include "MassEntityUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MassZoneGraphNavigationTrait)


void UMassZoneGraphNavigationTrait::BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const
{
	FMassEntityManager& EntityManager = UE::Mass::Utils::GetEntityManagerChecked(World);

	BuildContext.RequireFragment<FAgentRadiusFragment>();
	BuildContext.RequireFragment<FTransformFragment>();
	BuildContext.RequireFragment<FMassVelocityFragment>();
	BuildContext.RequireFragment<FMassMoveTargetFragment>();

	BuildContext.AddFragment<FMassZoneGraphLaneLocationFragment>();
	BuildContext.AddFragment<FMassZoneGraphPathRequestFragment>();
	BuildContext.AddFragment<FMassZoneGraphShortPathFragment>();
	BuildContext.AddFragment<FMassZoneGraphCachedLaneFragment>();
	BuildContext.AddFragment<FMassLaneCacheBoundaryFragment>();

	const FConstSharedStruct ZGMovementParamsFragment = EntityManager.GetOrCreateConstSharedFragment(NavigationParameters);
	BuildContext.AddConstSharedFragment(ZGMovementParamsFragment);
}
