// Copyright Epic Games, Inc. All Rights Reserved.

#include "MassZoneGraphNavigationUtils.h"
#include "MassCommonTypes.h"
#include "MassNavigationFragments.h"
#include "MassZoneGraphNavigationFragments.h"
#include "ZoneGraphSubsystem.h"
#include "VisualLogger/VisualLogger.h"
#include "MassDebugger.h"

namespace UE::MassNavigation
{
	static constexpr float InflateDistance = 200.0f; // @todo: make a setting.

	bool ActivateActionMove(const UWorld& World,
							const UObject* Requester,
							const FMassEntityHandle Entity,
							const UZoneGraphSubsystem& ZoneGraphSubsystem,
							const FMassZoneGraphLaneLocationFragment& LaneLocation,
							const FZoneGraphShortPathRequest& PathRequest,
							const float AgentRadius,
							const float DesiredSpeed,
							FMassMoveTargetFragment& InOutMoveTarget,
							FMassZoneGraphShortPathFragment& OutShortPath,
							FMassZoneGraphCachedLaneFragment& OutCachedLane)
	{
		OutShortPath.Reset();
		OutCachedLane.Reset();
		InOutMoveTarget.DistanceToGoal = 0.0f;
		InOutMoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
		InOutMoveTarget.DesiredSpeed.Set(0.0f);

		if (!ensureMsgf(InOutMoveTarget.GetCurrentAction() == EMassMovementAction::Move, TEXT("Expecting action 'Move': Invalid action %u"), EnumToUnderlyingType(InOutMoveTarget.GetCurrentAction())))
		{
			return false;
		}

		const FZoneGraphStorage* ZoneGraphStorage = ZoneGraphSubsystem.GetZoneGraphStorage(LaneLocation.LaneHandle.DataHandle);
		if (ZoneGraphStorage == nullptr)
		{
			UE_VLOG(Requester, LogMassNavigation, Error, TEXT("Entity [%s] move request failed: missing ZoneGraph Storage for current lane %s."),
				*Entity.DebugGetDescription(),
				*LaneLocation.LaneHandle.ToString());
			return false;
		}

		InOutMoveTarget.IntentAtGoal = EMassMovementAction::Stand;
		InOutMoveTarget.DesiredSpeed.Set(DesiredSpeed);

		OutCachedLane.CacheLaneData(*ZoneGraphStorage, LaneLocation.LaneHandle, LaneLocation.DistanceAlongLane, PathRequest.TargetDistance, InflateDistance);
		if (OutShortPath.RequestPath(OutCachedLane, PathRequest, LaneLocation.DistanceAlongLane, AgentRadius))
		{
			InOutMoveTarget.IntentAtGoal = OutShortPath.EndOfPathIntent;
			InOutMoveTarget.DistanceToGoal = (OutShortPath.NumPoints > 0) ? OutShortPath.Points[OutShortPath.NumPoints - 1].DistanceAlongLane.Get() : 0.0f;
#if WITH_MASSGAMEPLAY_DEBUG
			UE_CVLOG(UE::Mass::Debug::IsDebuggingEntity(Entity),
				Requester,
				LogMassNavigation,
				Log,
				TEXT("Move %s, on lane %s, from %.1fcm to %.1fcm, next lane %s."),
				PathRequest.bMoveReverse ? TEXT("reverse") : TEXT("forward"),
				*LaneLocation.LaneHandle.ToString(),
				LaneLocation.DistanceAlongLane,
				PathRequest.TargetDistance,
				*PathRequest.NextLaneHandle.ToString());
#endif // WITH_MASSGAMEPLAY_DEBUG
		}
		else
		{
			UE_VLOG(Requester, LogMassNavigation, Error, TEXT("Entity [%s] move request failed: unable to request path on lane %s."),
				*Entity.DebugGetDescription(),
				*LaneLocation.LaneHandle.ToString());
			return false;
		}

		UE_VLOG(Requester, LogMassNavigation, Log, TEXT("Entity [%s] successfully requested %s"), *Entity.DebugGetDescription(), *InOutMoveTarget.ToString());
		return true;
	}

	bool ActivateActionStand(const UWorld& World,
							 const UObject* Requester,
							 const FMassEntityHandle Entity,
							 const UZoneGraphSubsystem& ZoneGraphSubsystem,
							 const FMassZoneGraphLaneLocationFragment& LaneLocation,
							 const float DesiredSpeed,
							 FMassMoveTargetFragment& MoveTarget,
							 FMassZoneGraphShortPathFragment& ShortPath,
							 FMassZoneGraphCachedLaneFragment& CachedLane)
	{
		ShortPath.Reset();
		CachedLane.Reset();
		MoveTarget.DistanceToGoal = 0.0f;
		MoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
		MoveTarget.DesiredSpeed.Set(0.0f);

		if (!ensureMsgf(MoveTarget.GetCurrentAction() == EMassMovementAction::Stand, TEXT("Expecting action 'Stand': Invalid action %u"), EnumToUnderlyingType(MoveTarget.GetCurrentAction())))
		{
			return false;
		}

		const FZoneGraphStorage* ZoneGraphStorage = ZoneGraphSubsystem.GetZoneGraphStorage(LaneLocation.LaneHandle.DataHandle);

		MoveTarget.IntentAtGoal = EMassMovementAction::Stand;
		MoveTarget.DesiredSpeed.Set(DesiredSpeed);

		CachedLane.CacheLaneData(*ZoneGraphStorage, LaneLocation.LaneHandle, LaneLocation.DistanceAlongLane, LaneLocation.DistanceAlongLane, InflateDistance);

		UE_VLOG(Requester, LogMassNavigation, Log, TEXT("Entity [%s] successfully requested %s"), *Entity.DebugGetDescription(), *MoveTarget.ToString());
		return true;
	}

	bool ActivateActionAnimate(const UWorld& World,
							   const UObject* Requester,
							   const FMassEntityHandle Entity,
							   FMassMoveTargetFragment& MoveTarget)
	{
		MoveTarget.DistanceToGoal = 0.0f;
		MoveTarget.EntityDistanceToGoal = FMassMoveTargetFragment::UnsetDistance;
		MoveTarget.DesiredSpeed.Set(0.0f);

		if (!ensureMsgf(MoveTarget.GetCurrentAction() == EMassMovementAction::Animate, TEXT("Expecting action 'Animate': Invalid action %u"), EnumToUnderlyingType(MoveTarget.GetCurrentAction())))
		{
			return false;
		}

		MoveTarget.IntentAtGoal = EMassMovementAction::Stand;

		UE_VLOG(Requester, LogMassNavigation, Log, TEXT("Entity [%s] successfully requested %s"), *Entity.DebugGetDescription(), *MoveTarget.ToString());
		return true;
	}
}
