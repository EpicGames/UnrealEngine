// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassEntityTypes.h"
#include "ZoneGraphTypes.h"
#include "MassCommonTypes.h"
#include "MassZoneGraphNavigationTypes.h"
#include "Containers/StaticArray.h"
#include "MassZoneGraphNavigationFragments.generated.h"

#define UE_API MASSZONEGRAPHNAVIGATION_API


USTRUCT()
struct FMassZoneGraphNavigationParameters : public FMassConstSharedFragment
{
	GENERATED_BODY()

	/** Filter describing which lanes can be used when spawned. */
	UPROPERTY(EditAnywhere, Category="Navigation")
	FZoneGraphTagFilter LaneFilter;

	/** Query radius when trying to find nearest lane when spawned. */
	UPROPERTY(EditAnywhere, Category="Navigation", meta = (UIMin = 0.0, ClampMin = 0.0, ForceUnits="cm"))
	float QueryRadius = 500.0f;
};


/** Stores path request associated to a new movement action. This is used to replicate actions. */
USTRUCT()
struct FMassZoneGraphPathRequestFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Short path request Handle to current lane. */
	UPROPERTY(Transient)
	FZoneGraphShortPathRequest PathRequest;
};

/** Describes current location on ZoneGraph */ 
USTRUCT()
struct FMassZoneGraphLaneLocationFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Handle to current lane. */
	FZoneGraphLaneHandle LaneHandle;
	
	/** Distance along current lane. */
	float DistanceAlongLane = 0.0f;
	
	/** Cached lane length, used for clamping and testing if at end of lane. */
	float LaneLength = 0.0f;
};

/** Describes part of a ZoneGraph lane. */
USTRUCT()
struct FMassZoneGraphCachedLaneFragment : public FMassFragment
{
	GENERATED_BODY()

	static constexpr uint8 MaxPoints = 5;

	void Reset()
	{
		LaneHandle.Reset();
		LaneLength = 0.0f;
		LaneWidth = FMassInt16Real(0.0f);
		NumPoints = 0;
	}

	/** Caches portion of a lane from ZoneGraph. */
	UE_API void CacheLaneData(const FZoneGraphStorage& ZoneGraphStorage, const FZoneGraphLaneHandle CurrentLaneHandle,
					   const float CurrentDistanceAlongLane, const float TargetDistanceAlongLane, const float InflateDistance);

	int32 FindSegmentIndexAtDistance(const float DistanceAlongPath) const
	{
		int32 SegmentIndex = 0;
		while (SegmentIndex < ((int32)NumPoints - 2))
		{
			if (DistanceAlongPath < LanePointProgressions[SegmentIndex + 1].Get())
			{
				break;
			}
			SegmentIndex++;
		}

		return SegmentIndex;
	}

	float GetInterpolationTimeOnSegment(const int32 SegmentIndex, const float DistanceAlongPath) const
	{
		check(SegmentIndex >= 0 && SegmentIndex <= (int32)NumPoints - 2);
		const float StartDistance = LanePointProgressions[SegmentIndex].Get();
		const float EndDistance = LanePointProgressions[SegmentIndex + 1].Get();
		const float SegLength = EndDistance - StartDistance;
		const float InvSegLength = SegLength > KINDA_SMALL_NUMBER ? 1.0f / SegLength : 0.0f;
		return FMath::Clamp((DistanceAlongPath - StartDistance) * InvSegLength, 0.0f, 1.0f);
	}
	
	void InterpolatePointAndTangentOnSegment(const int32 SegmentIndex, const float DistanceAlongPath, FVector& OutPoint, FVector& OutTangent) const
	{
		const float T = GetInterpolationTimeOnSegment(SegmentIndex, DistanceAlongPath);
		OutPoint = FMath::Lerp(LanePoints[SegmentIndex], LanePoints[SegmentIndex + 1], T);
		OutTangent = FVector(FMath::Lerp(LaneTangentVectors[SegmentIndex].Get(), LaneTangentVectors[SegmentIndex + 1].Get(), T), 0.0f);
	}

	FVector InterpolatePointOnSegment(const int32 SegmentIndex, const float DistanceAlongPath) const
	{
		const float T = GetInterpolationTimeOnSegment(SegmentIndex, DistanceAlongPath);
		return FMath::Lerp(LanePoints[SegmentIndex], LanePoints[SegmentIndex + 1], T);
	}

	void GetPointAndTangentAtDistance(const float DistanceAlongPath, FVector& OutPoint, FVector& OutTangent) const
	{
		if (NumPoints == 0)
		{
			OutPoint = FVector::ZeroVector;
			OutTangent = FVector::ForwardVector;
			return;
		}
		if (NumPoints == 1)
		{
			OutPoint = LanePoints[0];
			OutTangent = FVector(LaneTangentVectors[0].Get(), 0.0f);
			return;
		}

		const int32 SegmentIndex = FindSegmentIndexAtDistance(DistanceAlongPath);
		InterpolatePointAndTangentOnSegment(SegmentIndex, DistanceAlongPath, OutPoint, OutTangent);
	}

	FVector GetPointAtDistance(const float DistanceAlongPath) const
	{
		if (NumPoints == 0)
		{
			return FVector::ZeroVector;
		}
		if (NumPoints == 1)
		{
			return LanePoints[0];
		}

		const int32 SegmentIndex = FindSegmentIndexAtDistance(DistanceAlongPath);
		return InterpolatePointOnSegment(SegmentIndex, DistanceAlongPath);
	}

	bool IsDistanceAtLaneExtrema(const float Distance) const
	{
		static constexpr float Epsilon = 0.1f;
		return Distance <= Epsilon || (Distance - LaneLength) >= -Epsilon;
	}

	FZoneGraphLaneHandle LaneHandle;
	
	/** Lane points */
	TStaticArray<FVector, MaxPoints> LanePoints;

	/** Cached length of the lane. */
	float LaneLength = 0.0f;

	/** Lane tangents */
	TStaticArray<FMassSnorm8Vector2D, MaxPoints> LaneTangentVectors;

	/** lane Advance distances */
	TStaticArray<FMassInt16Real10, MaxPoints> LanePointProgressions;

	/** Cached width of the lane. */
	FMassInt16Real LaneWidth = FMassInt16Real(0.0f);

	/** Additional space left of the lane */
	FMassInt16Real LaneLeftSpace = FMassInt16Real(0.0f);

	/** Additional space right of the lane */
	FMassInt16Real LaneRightSpace = FMassInt16Real(0.0f);

	/** ID incremented each time the cache is updated. */
	uint16 CacheID = 0;
	
	/** Number of points on path. */
	uint8 NumPoints = 0;
};

USTRUCT()
struct FMassZoneGraphPathPoint
{
	GENERATED_BODY()

	/** Position of the path. */
	FVector Position = FVector::ZeroVector;

	/** Tangent direction of the path. */
	FMassSnorm8Vector2D Tangent;

	/** Position of the point along the original path. (Could potentially be uint16 at 10cm accuracy) */
	FMassInt16Real10 DistanceAlongLane = FMassInt16Real10(0.0f);

	/** Distance along the offset path from first point. (Could potentially be uint16 at 10cm accuracy) */
	FMassInt16Real Distance = FMassInt16Real(0.0f);

	/** True if this point is assumed to be off lane. */
	uint8 bOffLane : 1;

	/** True if this point is lane start or end point. */
	uint8 bIsLaneExtrema : 1;
};

/** Describes short path along ZoneGraph */
// @todo MassMovement: it should be possible to prune this down to 64bytes
// - remove debug lane handle, and replace other with index
// - see if we can remove move tangent?
USTRUCT()
struct FMassZoneGraphShortPathFragment : public FMassFragment
{
	GENERATED_BODY()

	FMassZoneGraphShortPathFragment() = default;
	
	static constexpr uint8 MaxPoints = 3;

	void Reset()
	{
#if WITH_MASSGAMEPLAY_DEBUG
		DebugLaneHandle.Reset();
#endif
		NextLaneHandle.Reset();
		NextExitLinkType = EZoneLaneLinkType::None;
		ProgressDistance = 0.0f;
		NumPoints = 0;
		bMoveReverse = false;
		EndOfPathIntent = EMassMovementAction::Stand;
		bPartialResult = false;
		bDone = false;
	}

	/** Requests path along the current lane */
	UE_API bool RequestPath(const FMassZoneGraphCachedLaneFragment& CachedLane, const FZoneGraphShortPathRequest& Request, const float CurrentDistanceAlongLane, const float AgentRadius);

	/** Requests path to stand at current position. */
	UE_API bool RequestStand(const FMassZoneGraphCachedLaneFragment& CachedLane, const float CurrentDistanceAlongLane, const FVector& CurrentPosition);
	
	bool IsDone() const
	{
		// @todo MassMovement: should we remove NumPoints == 0? The logic used to be quite different when it was really needed.
		return NumPoints == 0 || bDone;
	}

#if WITH_MASSGAMEPLAY_DEBUG
	/** Current lane handle, for debug */
	FZoneGraphLaneHandle DebugLaneHandle;
#endif
	
	/** If valid, the this lane will be set as current lane after the path follow is completed. */
	FZoneGraphLaneHandle NextLaneHandle;
	
	/** Current progress distance along the lane. */
	float ProgressDistance = 0.0f;
	
	/** Path points */
	TStaticArray<FMassZoneGraphPathPoint, MaxPoints> Points;

	/** If next lane is set, this is how to reach the lane from current lane. */
	EZoneLaneLinkType NextExitLinkType = EZoneLaneLinkType::None;
	
	/** Number of points on path. */
	uint8 NumPoints = 0;
	
	/** Intent at the end of the path. */
	EMassMovementAction EndOfPathIntent = EMassMovementAction::Stand;

	/** True if we're moving reverse */
	uint8 bMoveReverse : 1;

	/** True if the path was partial. */
	uint8 bPartialResult : 1;

	/** True when path follow is completed. */
	uint8 bDone : 1;
};

USTRUCT()
struct FMassLaneCacheBoundaryFragment : public FMassFragment
{
	GENERATED_BODY()

	/** Last update position. */
	FVector LastUpdatePosition = FVector::ZeroVector;

	/** Lane cached ID at last update. */
	uint16 LastUpdateCacheID = 0;
};

#undef UE_API
