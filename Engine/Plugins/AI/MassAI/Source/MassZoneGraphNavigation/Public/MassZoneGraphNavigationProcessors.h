// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassProcessor.h"
#include "MassObserverProcessor.h"
#include "MassEntityQuery.h"
#include "MassZoneGraphNavigationProcessors.generated.h"

#define UE_API MASSZONEGRAPHNAVIGATION_API

class UMassSignalSubsystem;


/**
 * Processor for initializing nearest location on ZoneGraph.
 */
UCLASS(MinimalAPI)
class UMassZoneGraphLocationInitializer : public UMassObserverProcessor
{
	GENERATED_BODY()
	
public:
	UE_API UMassZoneGraphLocationInitializer();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery;
};

/** 
 * Processor for updating move target on ZoneGraph path.
 */
UCLASS(MinimalAPI)
class UMassZoneGraphPathFollowProcessor : public UMassProcessor
{
	GENERATED_BODY()

protected:
	UE_API UMassZoneGraphPathFollowProcessor();

	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

	FMassEntityQuery EntityQuery_Conditional;
};

/** ZoneGraph lane cache boundary processor */
// @todo MassMovement: Make this signal based.
UCLASS(MinimalAPI)
class UMassZoneGraphLaneCacheBoundaryProcessor : public UMassProcessor
{
	GENERATED_BODY()

public:
	UE_API UMassZoneGraphLaneCacheBoundaryProcessor();

protected:
	UE_API virtual void ConfigureQueries(const TSharedRef<FMassEntityManager>& EntityManager) override;
	UE_API virtual void Execute(FMassEntityManager& EntityManager, FMassExecutionContext& Context) override;

private:
	FMassEntityQuery EntityQuery;
};

#undef UE_API
