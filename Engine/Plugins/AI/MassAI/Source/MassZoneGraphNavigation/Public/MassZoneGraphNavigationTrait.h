// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once 

#include "MassEntityTraitBase.h"
#include "MassZoneGraphNavigationFragments.h"
#include "MassZoneGraphNavigationTrait.generated.h"

#define UE_API MASSZONEGRAPHNAVIGATION_API


UCLASS(MinimalAPI, meta = (DisplayName = "ZoneGraph Navigation"))
class UMassZoneGraphNavigationTrait : public UMassEntityTraitBase
{
	GENERATED_BODY()

protected:
	UE_API virtual void BuildTemplate(FMassEntityTemplateBuildContext& BuildContext, const UWorld& World) const override;

	UPROPERTY(Category="Movement", EditAnywhere)
	FMassZoneGraphNavigationParameters NavigationParameters;
};

#undef UE_API
