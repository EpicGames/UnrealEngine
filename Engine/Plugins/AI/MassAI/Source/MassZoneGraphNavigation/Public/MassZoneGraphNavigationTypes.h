// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MassCommonTypes.h"
#include "MassNavigationTypes.h"
#include "ZoneGraphTypes.h"
#include "MassZoneGraphNavigationTypes.generated.h"

/** Describes path request along one ZoneGraph lane. If the NextLaneHandle is set, lane is changed when path finishes. */
USTRUCT()
struct FZoneGraphShortPathRequest
{
	GENERATED_BODY()

	FZoneGraphShortPathRequest()
		: bMoveReverse(false)
		, bIsEndOfPathPositionSet(false)
		, bIsEndOfPathDirectionSet(false)
	{
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("%s to distance %.1f Next lane: %s of type %s. End of path intent:%s"),
			bMoveReverse ? TEXT("forward") : TEXT("reverse"), TargetDistance,
			NextLaneHandle.IsValid() ? *NextLaneHandle.ToString() : TEXT("unset"),
			NextLaneHandle.IsValid() ? *UEnum::GetValueAsString(NextExitLinkType) : TEXT("Unset"),
			*UEnum::GetValueAsString(EndOfPathIntent));
	}

	/** Position used as the start of the path*/
	UPROPERTY(Transient)
	FVector StartPosition = FVector::ZeroVector;

	/** Optional specific point at the end of the path. */
	UPROPERTY(Transient)
	FVector EndOfPathPosition = FVector::ZeroVector;

	/** If set, the lane will be switched when path finishes. */
	UPROPERTY(Transient)
	FZoneGraphLaneHandle NextLaneHandle;

	/** Distance to move. */
	UPROPERTY(Transient)
	float TargetDistance = 0.0f;

	/** Optional specific direction at the end of the path. Used only if EndOfPathPosition is set. */
	UPROPERTY(Transient)
	FMassSnorm8Vector EndOfPathDirection;

	/** If start or end of path is off-lane, the distance along the lane is pushed forward/back along the lane to make smoother transition. */
	UPROPERTY(Transient)
	FMassInt16Real AnticipationDistance = FMassInt16Real(50.f);

	UPROPERTY(Transient)
	FMassInt16Real EndOfPathOffset = FMassInt16Real(0.0f);

	/** Movement intent at the end of the path. */
	UPROPERTY(Transient)
	EMassMovementAction EndOfPathIntent = EMassMovementAction::Stand;

	/** How the NextLaneHandle links to current lane. */
	UPROPERTY(Transient)
	EZoneLaneLinkType NextExitLinkType = EZoneLaneLinkType::None;

	/** If true, move backwards along the lane. */
	UPROPERTY(Transient)
	uint8 bMoveReverse : 1;

	/** Indicates if the optional end of path position is set. */
	UPROPERTY(Transient)
	uint8 bIsEndOfPathPositionSet : 1;

	/** Indicates if the optional end of path direction is set. Used only if EndOfPathPosition is set. */
	UPROPERTY(Transient)
	uint8 bIsEndOfPathDirectionSet : 1;
};
