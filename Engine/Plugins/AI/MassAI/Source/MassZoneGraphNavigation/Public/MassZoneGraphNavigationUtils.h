// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "MassEntityTypes.h"
#endif // UE_ENABLE_INCLUDE_ORDER_DEPRECATED_IN_5_6
#include "Mass/EntityHandle.h"

struct FZoneGraphShortPathRequest;
struct FMassZoneGraphCachedLaneFragment;
struct FMassZoneGraphShortPathFragment;
struct FMassMoveTargetFragment;
struct FMassZoneGraphPathRequestFragment;
struct FMassZoneGraphLaneLocationFragment;
class UMassSignalSubsystem;
class UZoneGraphSubsystem;
class UZoneGraphAnnotationSubsystem;

namespace UE::MassNavigation
{
	MASSZONEGRAPHNAVIGATION_API bool ActivateActionMove(const UWorld& World,
											 const UObject* Requester,
											 const FMassEntityHandle Entity,
											 const UZoneGraphSubsystem& ZoneGraphSubsystem,
											 const FMassZoneGraphLaneLocationFragment& LaneLocation,
											 const FZoneGraphShortPathRequest& PathRequest,
											 const float AgentRadius,
											 const float DesiredSpeed,
											 FMassMoveTargetFragment& InOutMoveTarget,
											 FMassZoneGraphShortPathFragment& OutShortPath,
											 FMassZoneGraphCachedLaneFragment& OutCachedLane);

	MASSZONEGRAPHNAVIGATION_API bool ActivateActionStand(const UWorld& World,
											  const UObject* Requester,
											  const FMassEntityHandle Entity,
											  const UZoneGraphSubsystem& ZoneGraphSubsystem,
											  const FMassZoneGraphLaneLocationFragment& LaneLocation,
											  const float DesiredSpeed,
											  FMassMoveTargetFragment& MoveTarget,
											  FMassZoneGraphShortPathFragment& ShortPath,
											  FMassZoneGraphCachedLaneFragment& CachedLane);

	MASSZONEGRAPHNAVIGATION_API bool ActivateActionAnimate(const UWorld& World,
												const UObject* Requester,
												const FMassEntityHandle Entity,
												FMassMoveTargetFragment& MoveTarget);
};
